{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Mesh LOD Toolset",
	"Description": "A set of modules implementing 3D mesh LOD creation",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Hidden": true,
	"Modules": [
		{
			"Name": "MeshLODToolset",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
	],
	"Plugins": [
		{
			"Name": "GeometryProcessing",
			"Enabled": true
		},
		{
			"Name": "GeometryFlow",
			"Enabled": true
		},
		{
			"Name": "MeshModelingToolsetExp",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		},
		{
			"Name": "EditorScriptingUtilities",
			"Enabled": true
		}
	]
}
