// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MeshLODToolset : ModuleRules
{
	public MeshLODToolset(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicIncludePaths.AddRange(
			new string[]
			{
				// ... add public include paths required here ...
			}
		);

		PrivateIncludePaths.AddRange(
			new string[]
			{
				// ... add other private include paths required here ...
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"InteractiveToolsFramework",
				"GeometryCore",
				"DynamicMesh",
				"MeshConversion",
				"MeshConversionEngineTypes",
				"MeshDescription",
				"StaticMeshDescription",
				"ModelingComponents",
				"ModelingComponentsEditorOnly",
				"MeshModelingToolsExp",
				"GeometryFlowCore",
				"GeometryFlowMeshProcessing",
				"GeometryFlowMeshProcessingEditor",
				// ... add other public dependencies that you statically link with here ...
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"RenderCore",
				"UnrealEd",
				"EditorScriptingUtilities",
				"Slate",
				"SlateCore",
				"AssetDefinition"
				// ... add private dependencies that you statically link with here ...
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
		);
	}
}