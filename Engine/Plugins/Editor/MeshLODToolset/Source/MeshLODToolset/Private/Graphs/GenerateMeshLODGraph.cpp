// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graphs/GenerateMeshLODGraph.h"

#include "MeshLODToolsetModule.h"
#include "GeometryFlowGraphUtil.h"
#include "BaseNodes/TransferNode.h"
#include "BaseNodes/SwitchNode.h"

#include "MeshProcessingNodes/MeshDeleteTrianglesNode.h"
#include "MeshProcessingNodes/CompactMeshNode.h"
#include "MeshProcessingNodes/TransferMeshMaterialIDsNode.h"

#include "MeshProcessingNodes/MeshTangentsNodes.h"

#include "MeshProcessingNodes/MeshRepackUVsNode.h"

#include "MeshBakingNodes/BakeMeshNormalMapNode.h"
#include "MeshBakingNodes/BakeMeshMultiTextureNode.h"

#include "MeshDecompositionNodes/MakeTriangleSetsNode.h"
#include "MeshProcessingNodeRegistration.h"
#include "MeshProcessingEditorNodeRegistration.h"
#include "GeometryFlowCoreNodeRegistration.h"

using namespace UE::Geometry;
using namespace UE::GeometryFlow;


typedef UE::GeometryFlow::TSwitchNode<FDynamicMesh3, 4, (int)EMeshProcessingDataTypes::DynamicMesh> FMeshGeneratorSwitchNode;


void FGenerateMeshLODGraph::SetSourceMesh(const FDynamicMesh3& SourceMeshIn)
{
	UpdateSourceNodeValue<FDynamicMeshSourceNode>(*Graph, MeshSourceNode, SourceMeshIn);
}


void FGenerateMeshLODGraph::UpdatePreFilterSettings(const FMeshLODGraphPreFilterSettings& PreFilterSettings)
{
	FName FilterGroupLayerName = { PreFilterSettings.FilterGroupLayerName };
	UpdateSourceNodeValue<FNameSourceNode>(*Graph, FilterGroupsLayerNameNode, FilterGroupLayerName);
	CurrentPreFilterSettings = PreFilterSettings;
}

void FGenerateMeshLODGraph::UpdateCoreMeshGeneratorMode(ECoreMeshGeneratorMode NewMode)
{
	UpdateSwitchNodeInputIndex<FMeshGeneratorSwitchNode>(*Graph, MeshGeneratorSwitchNode, static_cast<int32>(NewMode) );
	CurrentCoreMeshGeneratorMode = NewMode;
}


void FGenerateMeshLODGraph::UpdateSolidifySettings(const FMeshSolidifySettings& SolidifySettings)
{
	UpdateSettingsSourceNodeValue(*Graph, SolidifySettingsNode, SolidifySettings);
	CurrentSolidifySettings = SolidifySettings;
}

void FGenerateMeshLODGraph::UpdateMorphologySettings(const FVoxClosureSettings& MorphologySettings)
{
	UpdateSettingsSourceNodeValue(*Graph, MorphologySettingsNode, MorphologySettings);
	CurrentMorphologySettings = MorphologySettings;
}

void FGenerateMeshLODGraph::UpdateMeshCleaningSettings(const FMeshMakeCleanGeometrySettings& CleaningSettings)
{
	UpdateSettingsSourceNodeValue(*Graph, CleanMeshSettingsNode, CleaningSettings);
	CurrentCleanMeshSettings = CleaningSettings;
}

void FGenerateMeshLODGraph::UpdateSimplifySettings(const FMeshSimplifySettings& SimplifySettings)
{
	UpdateSettingsSourceNodeValue(*Graph, SimplifySettingsNode, SimplifySettings);
	CurrentSimplifySettings = SimplifySettings;
}

void FGenerateMeshLODGraph::UpdateGenerateConvexHullMeshSettings(const FGenerateConvexHullMeshSettings& GenerateConvexHullMeshSettings)
{
	UpdateSettingsSourceNodeValue(*Graph, GenerateConvexHullMeshSettingsNode, GenerateConvexHullMeshSettings);
	CurrentGenerateConvexHullMeshSettings = GenerateConvexHullMeshSettings;
}

void FGenerateMeshLODGraph::UpdateNormalsSettings(const FMeshNormalsSettings& NormalsSettings)
{
	UpdateSettingsSourceNodeValue(*Graph, NormalsSettingsNode, NormalsSettings);
	CurrentNormalsSettings = NormalsSettings;
}

void FGenerateMeshLODGraph::UpdateAutoUVSettings(const UE::GeometryFlow::FMeshAutoGenerateUVsSettings& AutoUVSettings)
{
	UpdateSettingsSourceNodeValue(*Graph, AutoUVSettingsNode, AutoUVSettings);
	CurrentAutoUVSettings = AutoUVSettings;
}

void FGenerateMeshLODGraph::UpdateBakeCacheSettings(const UE::GeometryFlow::FMeshMakeBakingCacheSettings& BakeCacheSettings)
{
	UpdateSettingsSourceNodeValue(*Graph, BakeCacheSettingsNode, BakeCacheSettings);
	CurrentBakeCacheSettings = BakeCacheSettings;
}


void FGenerateMeshLODGraph::UpdateGenerateSimpleCollisionSettings(const FGenerateSimpleCollisionSettings& GenSimpleCollisionSettings)
{
	UpdateSettingsSourceNodeValue(*Graph, GenerateSimpleCollisionSettingsNode, GenSimpleCollisionSettings);
	CurrentGenerateSimpleCollisionSettings = GenSimpleCollisionSettings;
}

void FGenerateMeshLODGraph::UpdateThickenWeightMap(const TArray<float>& ThickenWeightMap)
{
	FWeightMap WeightMap;
	WeightMap.Weights = ThickenWeightMap;
	UpdateSourceNodeValue<FWeightMapSourceNode>(*Graph, ThickenWeightMapNode, WeightMap);
}

void FGenerateMeshLODGraph::UpdateThickenSettings(const UE::GeometryFlow::FMeshThickenSettings& ThickenSettings)
{
	UpdateSettingsSourceNodeValue(*Graph, ThickenSettingsNode, ThickenSettings);
	CurrentThickenSettings = ThickenSettings;
}

void FGenerateMeshLODGraph::UpdateCollisionGroupLayerName(const FName& NewCollisionGroupLayerName)
{
	CollisionGroupLayerName = NewCollisionGroupLayerName;
	UpdateSourceNodeValue<FNameSourceNode>(*Graph, GroupLayerNameNode, CollisionGroupLayerName);
}


void FGenerateMeshLODGraph::EvaluateResult(
	FDynamicMesh3& ResultMesh,
	FMeshTangentsd& ResultTangents,
	FSimpleShapeSet3d& ResultCollision,
	UE::GeometryFlow::FNormalMapImage& NormalMap,
	TArray<TUniquePtr<UE::GeometryFlow::FTextureImage>>& TextureImages,
	UE::GeometryFlow::FTextureImage& MultiTextureImage,
	FProgressCancel* Progress)
{
	//FScopedDurationTimeLogger Timer(TEXT("FGenerateMeshLODGraph::EvaluateResult -- serial execution"));

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	//
	// evaluate normal map
	//

	NormalMap = FNormalMapImage();
	TUniquePtr<FEvaluationInfo> NormalMapEvalInfo = MakeUnique<FEvaluationInfo>();
	NormalMapEvalInfo->Progress = Progress;
	EGeometryFlowResult NormalMapEvalResult = Graph->EvaluateResult(BakeNormalMapNode, FBakeMeshNormalMapNode::OutParamNormalMap(),
		NormalMap, (int)EMeshProcessingDataTypes::NormalMapImage, NormalMapEvalInfo, true);

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	ensure(NormalMapEvalResult == EGeometryFlowResult::Ok);
	UE_LOGF(LogMeshLODToolset, Display, "NormalMapPass - Evaluated %d Nodes, Recomputed %d", NormalMapEvalInfo->NumEvaluations(), NormalMapEvalInfo->NumComputes());


	//
	// evaluate transferred textures
	//

	for (FBakeTextureGraphInfo& TexBakeStep : BakeTextureNodes)
	{
		TUniquePtr<UE::GeometryFlow::FTextureImage> NewImage = MakeUnique<UE::GeometryFlow::FTextureImage>();
		TUniquePtr<FEvaluationInfo> TexBakeEvalInfo = MakeUnique<FEvaluationInfo>();
		TexBakeEvalInfo->Progress = Progress;
		EGeometryFlowResult TexBakeEvalResult = Graph->EvaluateResult(TexBakeStep.BakeNode, FBakeMeshTextureImageNode::OutParamTextureImage(),
			*NewImage, (int)EMeshProcessingDataTypes::TextureImage, TexBakeEvalInfo, true);

		if (Progress && Progress->Cancelled())
		{
			return;
		}

		TextureImages.Add(MoveTemp(NewImage));
		ensure(TexBakeEvalResult == EGeometryFlowResult::Ok);

		UE_LOGF(LogMeshLODToolset, Display, "TextureBakePass %ls - Evaluated %d Nodes, Recomputed %d", *TexBakeStep.Identifier, TexBakeEvalInfo->NumEvaluations(), TexBakeEvalInfo->NumComputes());
	}


	// 
	// evaluate multi texture bake
	//
	{
		TUniquePtr<FEvaluationInfo> MultiTextureBakeEvalInfo = MakeUnique<FEvaluationInfo>();
		MultiTextureBakeEvalInfo->Progress = Progress;
		EGeometryFlowResult MultiTextureBakeEvalResult = Graph->EvaluateResult(BakeMultiTextureNode, FBakeMeshMultiTextureNode::OutParamTextureImage(),
																			   MultiTextureImage, (int)EMeshProcessingDataTypes::TextureImage, MultiTextureBakeEvalInfo, true);

		if (Progress && Progress->Cancelled())
		{
			return;
		}

		ensure(MultiTextureBakeEvalResult == EGeometryFlowResult::Ok);
		UE_LOGF(LogMeshLODToolset, Display, "MultiTextureBake - Evaluated %d Nodes, Recomputed %d", MultiTextureBakeEvalInfo->NumEvaluations(), MultiTextureBakeEvalInfo->NumComputes());
	}

	// 
	// evaluate tangents
	//

	bool bTakeResultTangents = false;
	ResultTangents = FMeshTangentsd();

	TUniquePtr<FEvaluationInfo> TangentsEvalInfo = MakeUnique<FEvaluationInfo>();
	TangentsEvalInfo->Progress = Progress;
	EGeometryFlowResult TangentsEvalResult = Graph->EvaluateResult(TangentsOutputNode, FMeshTangentsTransferNode::OutParamValue(),
		ResultTangents, (int)EMeshProcessingDataTypes::MeshTangentSet, TangentsEvalInfo, bTakeResultTangents);

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	ensure(TangentsEvalResult == EGeometryFlowResult::Ok);
	UE_LOGF(LogMeshLODToolset, Display, "OutputTangentsPass - Evaluated %d Nodes, Recomputed %d", TangentsEvalInfo->NumEvaluations(), TangentsEvalInfo->NumComputes());


	//
	// evaluate result mesh
	// 


	bool bTakeResultMesh = true;
	ResultMesh.Clear();

	TUniquePtr<FEvaluationInfo> MeshEvalInfo = MakeUnique<FEvaluationInfo>();
	MeshEvalInfo->Progress = Progress;
	EGeometryFlowResult EvalResult = Graph->EvaluateResult(MeshOutputNode, FDynamicMeshTransferNode::OutParamValue(),
		ResultMesh, (int32)EMeshProcessingDataTypes::DynamicMesh, MeshEvalInfo, bTakeResultMesh);

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	ensure(EvalResult == EGeometryFlowResult::Ok);
	UE_LOGF(LogMeshLODToolset, Display, "OutputMeshPass - Evaluated %d Nodes, Recomputed %d", MeshEvalInfo->NumEvaluations(), MeshEvalInfo->NumComputes());

	//
	// evaluate collision
	//

	bool bTakeResultCollision = false;
	ResultCollision = FSimpleShapeSet3d();

	TUniquePtr<FEvaluationInfo> CollisionEvalInfo = MakeUnique<FEvaluationInfo>();
	CollisionEvalInfo->Progress = Progress;
	EGeometryFlowResult CollisionEvalResult = Graph->EvaluateResult(CollisionOutputNode, FCollisionGeometryTransferNode::OutParamValue(),
																	ResultCollision, FCollisionGeometry::DataTypeIdentifier, CollisionEvalInfo, bTakeResultCollision);

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	ensure(CollisionEvalResult == EGeometryFlowResult::Ok);
	UE_LOGF(LogMeshLODToolset, Display, "OutputCollisionPass - Evaluated %d Nodes, Recomputed %d", CollisionEvalInfo->NumEvaluations(), CollisionEvalInfo->NumComputes());

}



static bool IsSingleMaterialMesh(const FDynamicMesh3& Mesh, int32& UniqueMaterialID)
{
	if (Mesh.HasAttributes() == false) return true;
	if (Mesh.Attributes()->HasMaterialID() == false) return true;

	const FDynamicMeshMaterialAttribute* MaterialIDs = Mesh.Attributes()->GetMaterialID();
	UniqueMaterialID = 0;
	bool bConstantInitialized = false;
	for (int32 TriangleID : Mesh.TriangleIndicesItr())
	{
		int32 MaterialID = MaterialIDs->GetValue(TriangleID);
		if (bConstantInitialized && MaterialID != UniqueMaterialID)
		{
			return false;
		}
		if (!bConstantInitialized)
		{
			bConstantInitialized = true;
			UniqueMaterialID = MaterialID;
		}
	}
	return true;
}



void FGenerateMeshLODGraph::BuildGraph(const FDynamicMesh3* SourceMeshHint)
{
	// does nothing if all the node types are already registered.
	{
		UE::GeometryFlow::FCoreNodeRegistration::RegisterNodes();
		UE::GeometryFlow::FMeshProcessingNodeRegistration::RegisterNodes();
		UE::GeometryFlow::FMeshProcessingEditorNodeRegistration::RegisterNodes();
	}

	// precompute hints (async?)
	int32 UniqueMaterialID = 0;
	bool bIsSingleMaterialMesh = (SourceMeshHint != nullptr) ? IsSingleMaterialMesh(*SourceMeshHint, UniqueMaterialID) : false;

	Graph = MakeUnique<FGraph>();

	MeshSourceNode = Graph->AddNodeOfType<FDynamicMeshSourceNode>(TEXT("SourceMesh"));

	// remove detail triangles

	FGraph::FHandle FilterGroupsNode = Graph->AddNodeOfType<FIndexSetsSourceNode>(TEXT("FilterGroups"));
	FilterGroupsLayerNameNode = Graph->AddNodeOfType<FNameSourceNode>(TEXT("FilterGroupsLayerNameSource"));

	FGraph::FHandle MakeFilterTriangleSetsNode = Graph->AddNodeOfType<FMakeTriangleSetsFromGroupsNode>(TEXT("MakeFilterTriangles"));
	Graph->InferConnection(MeshSourceNode, MakeFilterTriangleSetsNode);
	Graph->InferConnection(FilterGroupsNode, MakeFilterTriangleSetsNode);
	Graph->InferConnection(FilterGroupsLayerNameNode, MakeFilterTriangleSetsNode);

	FilterTrianglesNode = Graph->AddNodeOfType<FMeshDeleteTrianglesNode>(TEXT("FilterMesh"));
	Graph->InferConnection(MeshSourceNode, FilterTrianglesNode);
	Graph->InferConnection(MakeFilterTriangleSetsNode, FilterTrianglesNode);

	// generating low-poly mesh

	// optionally thicken some parts of the mesh before solidifying
	ThickenNode = Graph->AddNodeOfType<FMeshThickenNode>(TEXT("Thicken"));
	ThickenWeightMapNode = Graph->AddNodeOfType<FWeightMapSourceNode>(TEXT("ThickenWeightMapNode"));
	ThickenSettingsNode = Graph->AddNodeOfType<FThickenSettingsSourceNode>(TEXT("ThickenSettingsSource"));

	Graph->InferConnection(ThickenWeightMapNode, ThickenNode);
	Graph->InferConnection(ThickenSettingsNode, ThickenNode);
	Graph->InferConnection(FilterTrianglesNode, ThickenNode);

	SolidifyNode = Graph->AddNodeOfType<FSolidifyMeshNode>(TEXT("Solidify"));
	Graph->InferConnection(ThickenNode, SolidifyNode);
	SolidifySettingsNode = Graph->AddNodeOfType<FSolidifySettingsSourceNode>(TEXT("SolidifySettings"));
	Graph->InferConnection(SolidifySettingsNode, SolidifyNode);

	MorphologyNode = Graph->AddNodeOfType<FVoxClosureMeshNode>(TEXT("Closure"));
	Graph->InferConnection(SolidifyNode, MorphologyNode);
	MorphologySettingsNode = Graph->AddNodeOfType<FVoxClosureSettingsSourceNode>(TEXT("ClosureSettings"));
	Graph->InferConnection(MorphologySettingsNode, MorphologyNode);

	CleanMeshNode = Graph->AddNodeOfType<FMeshMakeCleanGeometryNode>(TEXT("Cleaner"));
	Graph->InferConnection(ThickenNode, CleanMeshNode);
	CleanMeshSettingsNode = Graph->AddNodeOfType<FMeshMakeCleanGeometrySettingsSourceNode>(TEXT("CleanerSettings"));
	Graph->InferConnection(CleanMeshSettingsNode, CleanMeshNode);

	GenerateConvexHullMeshNode = Graph->AddNodeOfType<FGenerateConvexHullMeshNode>(TEXT("ConvexHullMesh"));
	Graph->InferConnection(FilterTrianglesNode, GenerateConvexHullMeshNode);
	GenerateConvexHullMeshSettingsNode = Graph->AddNodeOfType<FGenerateConvexHullMeshSettingsSourceNode>(TEXT("ConvexHullMeshSettings"));
	Graph->InferConnection(GenerateConvexHullMeshSettingsNode, GenerateConvexHullMeshNode);


	// integer param that controls which mesh generator will be used
	MeshGeneratorSwitchNode = Graph->AddNodeOfType<FMeshGeneratorSwitchNode>(TEXT("MeshGeneratorSwitch"));
	Graph->AddConnection(SolidifyNode, FSolidifyMeshNode::OutParamResultMesh(), MeshGeneratorSwitchNode, FMeshGeneratorSwitchNode::InParamValue(0));
	Graph->AddConnection(MorphologyNode, FVoxClosureMeshNode::OutParamResultMesh(), MeshGeneratorSwitchNode, FMeshGeneratorSwitchNode::InParamValue(1));
	Graph->AddConnection(CleanMeshNode, FMeshMakeCleanGeometryNode::OutParamResultMesh(), MeshGeneratorSwitchNode, FMeshGeneratorSwitchNode::InParamValue(2));
	Graph->AddConnection(GenerateConvexHullMeshNode, FGenerateConvexHullMeshNode::OutParamResultMesh(), MeshGeneratorSwitchNode, FMeshGeneratorSwitchNode::InParamValue(3));

	// helper to separate next block from changes above
	FGraph::FHandle FinalMeshOutputNode = MeshGeneratorSwitchNode;

	// skip MaterialID projection/transfer if the input mesh only had one material
	FGraph::FHandle SimplifyPhaseInputNode = FinalMeshOutputNode;
	if (bIsSingleMaterialMesh == false)
	{
		FGraph::FHandle MatIDTransferNode = Graph->AddNodeOfType<FTransferMeshMaterialIDsNode>(TEXT("TransferMaterialIDs"));
		Graph->AddConnection(MeshSourceNode, FDynamicMeshSourceNode::OutParamValue(), MatIDTransferNode, FTransferMeshMaterialIDsNode::InParamMaterialSourceMesh());
		Graph->InferConnection(FinalMeshOutputNode, MatIDTransferNode);
		SimplifyPhaseInputNode = MatIDTransferNode;
	}

	// need to compute valid normals before Simplify
	//FGraph::FHandle PerVertexNormalsNode = Graph->AddNodeOfType<FComputeMeshPerVertexOverlayNormalsNode>(TEXT("PerVertexOverlayNormals"));
	FGraph::FHandle PerVertexNormalsNode = Graph->AddNodeOfType<FComputeMeshPerVertexNormalsNode>(TEXT("PerVertexNormals"));
	Graph->InferConnection(SimplifyPhaseInputNode, PerVertexNormalsNode);

	SimplifyNode = Graph->AddNodeOfType<FSimplifyMeshNode>(TEXT("Simplify"));
	Graph->InferConnection(PerVertexNormalsNode, SimplifyNode);
	SimplifySettingsNode = Graph->AddNodeOfType<FSimplifySettingsSourceNode>(TEXT("SimplifySettings"));
	Graph->InferConnection(SimplifySettingsNode, SimplifyNode);

	FGraph::FHandle CompactNode = Graph->AddNodeOfType<FCompactMeshNode>(TEXT("Compact"));
	Graph->InferConnection(SimplifyNode, CompactNode);

	NormalsNode = Graph->AddNodeOfType<FComputeMeshNormalsNode>(TEXT("Normals"));
	Graph->InferConnection(CompactNode, NormalsNode);
	NormalsSettingsNode = Graph->AddNodeOfType<FNormalsSettingsSourceNode>(TEXT("NormalsSettings"));
	Graph->InferConnection(NormalsSettingsNode, NormalsNode);

	// computing UVs

	AutoUVNode = Graph->AddNodeOfType<FMeshAutoGenerateUVsNode>(TEXT("AutoUV"));
	Graph->InferConnection(NormalsNode, AutoUVNode);
	AutoUVSettingsNode = Graph->AddNodeOfType<FMeshAutoGenerateUVsSettingsSourceNode>(TEXT("AutoUVSettings"));
	Graph->InferConnection(AutoUVSettingsNode, AutoUVNode);

	// disable RecomputeUVNode for now
	//RecomputeUVNode = Graph->AddNodeOfType<FMeshRecalculateUVsNode>(TEXT("RecalcUV"));
	//Graph->InferConnection(AutoUVNode, RecomputeUVNode);
	//RecomputeUVSettingsNode = Graph->AddNodeOfType<FMeshRecalculateUVsSettingsSourceNode>(TEXT("RecalcUVSettings"));
	//Graph->InferConnection(RecomputeUVSettingsNode, RecomputeUVNode);

	RepackUVNode = Graph->AddNodeOfType<FMeshRepackUVsNode>(TEXT("RepackUV"));
	//Graph->InferConnection(RecomputeUVNode, RepackUVNode);
	Graph->InferConnection(AutoUVNode, RepackUVNode);
	RepackUVSettingsNode = Graph->AddNodeOfType<FMeshRepackUVsSettingsSourceNode>(TEXT("RepackUVSettings"));
	Graph->InferConnection(RepackUVSettingsNode, RepackUVNode);


	// final mesh output

	MeshOutputNode = Graph->AddNodeOfType<FDynamicMeshTransferNode>(TEXT("OutputMesh"));
	Graph->InferConnection(RepackUVNode, MeshOutputNode);


	// create tangents

	TangentsNode = Graph->AddNodeOfType<FComputeMeshTangentsNode>(TEXT("Tangents"));
	Graph->InferConnection(RepackUVNode, TangentsNode);
	TangentsSettingsNode = Graph->AddNodeOfType<FTangentsSettingsSourceNode>(TEXT("TangentsSettings"));
	Graph->InferConnection(TangentsSettingsNode, TangentsNode);

	// tangents output
	TangentsOutputNode = Graph->AddNodeOfType<FMeshTangentsTransferNode>(TEXT("OutputTangents"));
	Graph->InferConnection(TangentsNode, TangentsOutputNode);

	// create bake cache

	BakeCacheNode = Graph->AddNodeOfType<FMakeMeshBakingCacheNode>(TEXT("MakeBakeCache"));
	Graph->AddConnection(MeshSourceNode, FDynamicMeshSourceNode::OutParamValue(), BakeCacheNode, FMakeMeshBakingCacheNode::InParamDetailMesh());
	Graph->AddConnection(RepackUVNode, FMeshRepackUVsNode::OutParamResultMesh(), BakeCacheNode, FMakeMeshBakingCacheNode::InParamTargetMesh());
	BakeCacheSettingsNode = Graph->AddNodeOfType<FMeshMakeBakingCacheSettingsSourceNode>(TEXT("BakeCacheSettings"));
	Graph->InferConnection(BakeCacheSettingsNode, BakeCacheNode);

	// normal map baker

	BakeNormalMapNode = Graph->AddNodeOfType<FBakeMeshNormalMapNode>(TEXT("BakeNormalMap"));
	Graph->InferConnection(BakeCacheNode, BakeNormalMapNode);
	Graph->InferConnection(TangentsNode, BakeNormalMapNode);
	BakeNormalMapSettingsNode = Graph->AddNodeOfType<FBakeMeshNormalMapSettingsSourceNode>(TEXT("BakeNormalMapSettings"));
	Graph->InferConnection(BakeNormalMapSettingsNode, BakeNormalMapNode);


	// collision generation

	FGraph::FHandle IgnoreGroupsForCollisionNode = Graph->AddNodeOfType<FIndexSetsSourceNode>(TEXT("CollisionIgnoreGroups"));

	//DecomposeMeshForCollisionNode = Graph->AddNodeOfType<FMakeTriangleSetsFromMeshNode>(TEXT("Decompose"));
	DecomposeMeshForCollisionNode = Graph->AddNodeOfType<FMakeTriangleSetsFromGroupsNode>(TEXT("Decompose"));
	Graph->InferConnection(FilterTrianglesNode, DecomposeMeshForCollisionNode);
	Graph->InferConnection(IgnoreGroupsForCollisionNode, DecomposeMeshForCollisionNode);

	GroupLayerNameNode = Graph->AddNodeOfType<FNameSourceNode>(TEXT("GroupLayerNameNode"));
	Graph->AddConnection(GroupLayerNameNode, FNameSourceNode::OutParamValue(), 
						 DecomposeMeshForCollisionNode, FMakeTriangleSetsFromGroupsNode::InParamGroupLayer());

	GenerateSimpleCollisionNode = Graph->AddNodeOfType<FGenerateSimpleCollisionNode>(TEXT("GenerateSimpleCollision"));
	Graph->InferConnection(FilterTrianglesNode, GenerateSimpleCollisionNode);
	Graph->InferConnection(DecomposeMeshForCollisionNode, GenerateSimpleCollisionNode);
	GenerateSimpleCollisionSettingsNode = Graph->AddNodeOfType<FGenerateSimpleCollisionSettingsSourceNode>(TEXT("GenerateSimpleCollisionSettings"));
	Graph->InferConnection(GenerateSimpleCollisionSettingsNode, GenerateSimpleCollisionNode);

	// final collision output

	CollisionOutputNode = Graph->AddNodeOfType<FCollisionGeometryTransferNode>(TEXT("OutputCollision"));
	Graph->InferConnection(GenerateSimpleCollisionNode, CollisionOutputNode);


	//
	// parameters
	//

	CurrentCoreMeshGeneratorMode = ECoreMeshGeneratorMode::SolidifyAndClose;
	UpdateSwitchNodeInputIndex<FMeshGeneratorSwitchNode>(*Graph, MeshGeneratorSwitchNode, static_cast<int32>(CurrentCoreMeshGeneratorMode) );

	FIndexSets IgnoreGroupsForDelete;
	IgnoreGroupsForDelete.AppendSet({ 0 });
	UpdateSettingsSourceNodeValue(*Graph, FilterGroupsNode, IgnoreGroupsForDelete);

	FMeshLODGraphPreFilterSettings PreFilterSettings;
	PreFilterSettings.FilterGroupLayerName = FName();
	UpdatePreFilterSettings(PreFilterSettings);

	UE::GeometryFlow::FMeshThickenSettings MeshThickenSettings;
	MeshThickenSettings.ThickenAmount = 10.0f;
	UpdateThickenSettings(MeshThickenSettings);

	FMeshSolidifySettings SolidifySettings;
	SolidifySettings.VoxelResolution = 128;
	UpdateSolidifySettings(SolidifySettings);

	FVoxClosureSettings MorphologySettings;
	MorphologySettings.VoxelResolution = SolidifySettings.VoxelResolution;
	MorphologySettings.Distance = 5.0;
	UpdateMorphologySettings(MorphologySettings);

	FMeshMakeCleanGeometrySettings CleanerSettings;
	CleanerSettings.bClearMaterialIDs = bIsSingleMaterialMesh;
	CleanerSettings.bOutputOverlayVertexNormals = false;
	UpdateMeshCleaningSettings(CleanerSettings);

	// TODO: can use simpler settings if there is only one material...
	FMeshSimplifySettings SimplifySettings;
	SimplifySettings.bDiscardAttributes = false;
	SimplifySettings.SimplifyType = ToUEnum(EMeshSimplifyType::AttributeAware);
	//SimplifySettings.TargetType = EMeshSimplifyTargetType::TriangleCount;
	//SimplifySettings.TargetCount = 500;
	SimplifySettings.TargetType = ToUEnum(EMeshSimplifyTargetType::GeometricDeviation);
	SimplifySettings.GeometricTolerance = 0.5;
	SimplifySettings.TargetCount = 500;
	SimplifySettings.MaterialBorderConstraints = ToUEnum(EEdgeRefineFlags::NoFlip);
	UpdateSimplifySettings(SimplifySettings);

	FMeshNormalsSettings NormalsSettings;
	NormalsSettings.NormalsType = ToUEnum(EComputeNormalsType::FromFaceAngleThreshold);
	NormalsSettings.AngleThresholdDeg = 60.0;
	UpdateNormalsSettings(NormalsSettings);

	FMeshAutoGenerateUVsSettings AutoUVSettings;
	AutoUVSettings.Method = ToUEnum(EAutoUVMethod::PatchBuilder);
	AutoUVSettings.UVAtlasNumCharts = 20;
	AutoUVSettings.UVAtlasStretch = 0.1;
	AutoUVSettings.XAtlasMaxIterations = 1;
	AutoUVSettings.NumInitialPatches = 100;
	AutoUVSettings.CurvatureAlignment = 1.0;
	AutoUVSettings.MergingThreshold = 1.5;
	AutoUVSettings.MaxAngleDeviationDeg = 45.0;
	AutoUVSettings.SmoothingSteps = 5;
	AutoUVSettings.SmoothingAlpha = 0.25;
	AutoUVSettings.bAutoPack = false;
	AutoUVSettings.PackingTargetWidth = 512;
	UpdateAutoUVSettings(AutoUVSettings);

	//FMeshRecalculateUVsSettings RecomputeUVSettings;
	//UpdateSettingsSourceNodeValue(*Graph, RecomputeUVSettingsNode, RecomputeUVSettings);

	FMeshRepackUVsSettings RepackUVSettings;
	UpdateSettingsSourceNodeValue(*Graph, RepackUVSettingsNode, RepackUVSettings);


	FMeshTangentsSettings TangentsSettings;
	UpdateSettingsSourceNodeValue(*Graph, TangentsSettingsNode, TangentsSettings);


	FMeshMakeBakingCacheSettings BakeCacheSettings;
	BakeCacheSettings.Dimensions = FIntPoint(1024, 1024);
	BakeCacheSettings.Thickness = 5.0;
	UpdateBakeCacheSettings(BakeCacheSettings);


	FBakeMeshNormalMapSettings NormalMapSettings;
	UpdateSettingsSourceNodeValue(*Graph, BakeNormalMapSettingsNode, NormalMapSettings);

	FIndexSets IgnoreGroupsForCollision;
	IgnoreGroupsForCollision.AppendSet({ 0 });
	UpdateSettingsSourceNodeValue(*Graph, IgnoreGroupsForCollisionNode, IgnoreGroupsForCollision);

	UpdateCollisionGroupLayerName(CollisionGroupLayerName);

	FGenerateSimpleCollisionSettings GenSimpleCollisionSettings;
	GenSimpleCollisionSettings.Type = EGeometryFlow_SimpleCollisionGeometryType::AlignedBoxes;
	UpdateGenerateSimpleCollisionSettings(GenSimpleCollisionSettings);

	TArray<float> Weights;
	UpdateThickenWeightMap(Weights);
}




int32 FGenerateMeshLODGraph::AppendTextureBakeNode(const TImageBuilder<FVector4f>& SourceImage, const FString& Identifier)
{
	FBakeTextureGraphInfo NewNode;
	NewNode.Index = BakeTextureNodes.Num();
	NewNode.Identifier = Identifier;

	// add source node
	NewNode.TexSourceNode = Graph->AddNodeOfType<FTextureImageSourceNode>(FString::Printf(TEXT("TextureSource%d_%s"), NewNode.Index, *NewNode.Identifier));

	// texture baker
	NewNode.BakeNode = Graph->AddNodeOfType<FBakeMeshTextureImageNode>(FString::Printf(TEXT("BakeTexImage%d_%s"), NewNode.Index, *NewNode.Identifier));
	ensure(Graph->InferConnection(BakeCacheNode, NewNode.BakeNode) == EGeometryFlowResult::Ok);
	ensure(Graph->InferConnection(NewNode.TexSourceNode, NewNode.BakeNode) == EGeometryFlowResult::Ok);
	FGraph::FHandle BakeTextureImageSettingsNode = Graph->AddNodeOfType<FBakeMeshTextureImageSettingsSourceNode>(TEXT("BakeTextureImageSettings"));
	ensure(Graph->InferConnection(BakeTextureImageSettingsNode, NewNode.BakeNode) == EGeometryFlowResult::Ok);

	FTextureImage InputTexImage;
	InputTexImage.Image = SourceImage;
	UpdateSourceNodeValue<FTextureImageSourceNode>(*Graph, NewNode.TexSourceNode, InputTexImage);

	BakeTextureNodes.Add(NewNode);

	return NewNode.Index;
}


void FGenerateMeshLODGraph::AppendMultiTextureBakeNode(const TMap<int32, TSafeSharedPtr<TImageBuilder<FVector4f>>>& SourceMaterialImages)
{
	// add source node
	MaterialIDTextureSourceNode = Graph->AddNodeOfType<FMaterialIDToTextureMapSourceNode>(FString::Printf(TEXT("MaterialIDTextureSource")));

	// texture baker
	BakeMultiTextureNode = Graph->AddNodeOfType<FBakeMeshMultiTextureNode>(FString::Printf(TEXT("BakeMultiTexture")));
	ensure(Graph->InferConnection(BakeCacheNode, BakeMultiTextureNode) == EGeometryFlowResult::Ok);
	ensure(Graph->InferConnection(MaterialIDTextureSourceNode, BakeMultiTextureNode) == EGeometryFlowResult::Ok);
	BakeMultiTextureSettingsNode = Graph->AddNodeOfType<FBakeMeshMultiTextureSettingsSourceNode>(TEXT("BakeMultiTextureSettings"));
	ensure(Graph->InferConnection(BakeMultiTextureSettingsNode, BakeMultiTextureNode) == EGeometryFlowResult::Ok);

	FMaterialIDToTextureMap InputMap;
	InputMap.MaterialIDTextureMap = SourceMaterialImages;

	UpdateSourceNodeValue<FMaterialIDToTextureMapSourceNode>(*Graph, MaterialIDTextureSourceNode, InputMap);
}
