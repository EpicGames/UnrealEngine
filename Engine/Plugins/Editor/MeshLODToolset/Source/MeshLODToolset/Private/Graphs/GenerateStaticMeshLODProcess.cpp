// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graphs/GenerateStaticMeshLODProcess.h"

#include "BodySetupEnums.h"
#include "MeshLODToolsetModule.h"

#include "Async/Async.h"

#include "Engine/StaticMeshSourceData.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "DynamicMeshToMeshDescription.h"

#include "AssetUtils/Texture2DUtil.h"
#include "AssetUtils/Texture2DBuilder.h"
#include "AssetUtils/MeshDescriptionUtil.h"

#include "Engine/Texture2D.h"
#include "Physics/PhysicsDataCollection.h"

#include "Graphs/GenerateMeshLODGraph.h"
#include "Misc/Paths.h"
#include "EditorAssetLibrary.h"
#include "AssetToolsModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "IAssetTools.h"
#include "Misc/PackageName.h"
#include "UObject/MetaData.h"

#include "Editor.h"

#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialExpressionTextureSampleParameter.h"
#include "Factories/MaterialInstanceConstantFactoryNew.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "MaterialGraph/MaterialGraphNode.h"
#include "WeightMapUtil.h"


#include "PhysicsEngine/BodySetup.h"
#include "RenderingThread.h"
#include "StaticMeshAttributes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GenerateStaticMeshLODProcess)

#define LOCTEXT_NAMESPACE "UGenerateStaticMeshLODProcess"

using namespace UE::Geometry;
using namespace UE::GeometryFlow;

namespace
{
#if WITH_EDITOR
	static EAsyncExecution GenerateSMLODAsyncExecTarget = EAsyncExecution::LargeThreadPool;
#else
	static EAsyncExecution GenerateSMLODAsyncExecTarget = EAsyncExecution::ThreadPool;
#endif
}



struct FReadTextureJob
{
	int32 MatIndex;
	int32 TexIndex;
};


namespace GenerateStaticMeshLODProcessHelpers
{
	// Given "xxxxx" returns "xxxxx_1"
	// Given "xxxxx_1" returns "xxxxx_2"
	// etc.
	// If anything goes wrong, just add "_1" to the end
	void AppendOrIncrementSuffix(FString& S)
	{
		TArray<FString> Substrings;
		S.ParseIntoArray(Substrings, TEXT("_"));
		if ( Substrings.Num() <= 1 ) 
		{
			S += TEXT("_1");
		}
		else
		{
			FString LastSubstring = Substrings.Last();
			int32 Num;
			bool bParsed = LexTryParseString(Num, *LastSubstring);
			if (bParsed)
			{
				++Num;
				Substrings.RemoveAt(Substrings.Num() - 1);
				S = FString::Join(Substrings, TEXT("_"));
				S += TEXT("_") + FString::FromInt(Num);
			}
			else
			{
				S += TEXT("_1");
			}
		}
	}

	TArray<int32> FindUnreferencedMaterials(UStaticMesh* StaticMesh, const FMeshDescription* MeshDescription)
	{
		const TArray<FStaticMaterial>& MaterialSet = StaticMesh->GetStaticMaterials();
		const int32 NumMaterials = MaterialSet.Num();

		auto IsValidMaterial = [&MaterialSet](int32 MaterialID) {
			return MaterialSet[MaterialID].MaterialInterface != nullptr;
		};

		// Initially flag only valid materials as potentially unused.
		TArray<bool> MatUnusedFlags;
		MatUnusedFlags.SetNum(NumMaterials);
		int32 NumMatUnused = 0;
		for (int32 MaterialID = 0; MaterialID < NumMaterials; ++MaterialID)
		{
			MatUnusedFlags[MaterialID] = IsValidMaterial(MaterialID);
			NumMatUnused += MatUnusedFlags[MaterialID];
		}

		TMap<FPolygonGroupID, int32> PolygonGroupToMaterialIndex;
		const FStaticMeshConstAttributes Attributes(*MeshDescription);
		TPolygonGroupAttributesConstRef<FName> PolygonGroupImportedMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();

		for (FPolygonGroupID PolygonGroupID : MeshDescription->PolygonGroups().GetElementIDs())
		{
			int32 MaterialIndex = StaticMesh->GetMaterialIndexFromImportedMaterialSlotName(PolygonGroupImportedMaterialSlotNames[PolygonGroupID]);
			if (MaterialIndex == INDEX_NONE)
			{
				MaterialIndex = PolygonGroupID.GetValue();
			}
			if (!MaterialSet.IsValidIndex(MaterialIndex))
			{
				MaterialIndex = 0;
			}
			PolygonGroupToMaterialIndex.Add(PolygonGroupID, MaterialIndex);
		}

		for (const FTriangleID TriangleID : MeshDescription->Triangles().GetElementIDs())
		{
			const FPolygonGroupID PolygonGroupID = MeshDescription->GetTrianglePolygonGroup(TriangleID);
			const int32 MaterialIndex = PolygonGroupToMaterialIndex[PolygonGroupID];
			bool& bMatUnusedFlag = MatUnusedFlags[MaterialIndex];
			NumMatUnused -= static_cast<int32>(bMatUnusedFlag);
			bMatUnusedFlag = false;
			if (NumMatUnused == 0)
			{
				break;
			}
		}

		TArray<int32> UnreferencedMaterials;
		UnreferencedMaterials.Reserve(NumMatUnused);
		for (int32 MaterialID = 0; MaterialID < NumMaterials; ++MaterialID)
		{
			if (MatUnusedFlags[MaterialID])
			{
				UnreferencedMaterials.Emplace(MaterialID);
			}
		}
		return UnreferencedMaterials;
	}
}

bool UGenerateStaticMeshLODProcess::Initialize(UStaticMesh* StaticMeshIn, FProgressCancel* Progress)
{
	if (!ensure(StaticMeshIn)) return false;
	if (!ensure(StaticMeshIn->GetNumSourceModels() > 0)) return false;

	// make sure we are not in rendering
	FlushRenderingCommands();

	SourceStaticMesh = StaticMeshIn;

	bUsingHiResSource = SourceStaticMesh->IsHiResMeshDescriptionValid();
	const FMeshDescription* UseSourceMeshDescription =
		(bUsingHiResSource) ? SourceStaticMesh->GetHiResMeshDescription() : SourceStaticMesh->GetMeshDescription(0);

	if (!UseSourceMeshDescription)
	{
		return false;
	}

	SourceMeshDescription = MakeShared<FMeshDescription>();
	*SourceMeshDescription = *UseSourceMeshDescription;

	// if not the high-res source, compute autogenerated normals/tangents
	if (bUsingHiResSource == false)
	{
		UE::MeshDescription::InitializeAutoGeneratedAttributes(*SourceMeshDescription, SourceStaticMesh, 0);
	}

	// start async mesh-conversion
	SourceMesh.Clear();
	TFuture<void> ConvertMesh = Async(GenerateSMLODAsyncExecTarget, [&]()
	{
		FMeshDescriptionToDynamicMesh GetSourceMesh;
		GetSourceMesh.Convert(SourceMeshDescription.Get(), SourceMesh);
	});

	// get list of source materials and find all the input texture params
	const TArray<FStaticMaterial>& Materials = SourceStaticMesh->GetStaticMaterials();

	// warn the user if there are any unsed materials in the mesh
	if (Progress)
	{
		for (int32 UnusedMaterialIndex : GenerateStaticMeshLODProcessHelpers::FindUnreferencedMaterials(SourceStaticMesh, SourceMeshDescription.Get()))
		{
			const TObjectPtr<class UMaterialInterface> MaterialInterface = Materials[UnusedMaterialIndex].MaterialInterface;
			if (ensure(MaterialInterface != nullptr))
			{
				FText WarningText = FText::Format(LOCTEXT("UnusedMaterialWarning", "Found an unused material ({0}). Consider removing it before using this tool."),
					                              FText::FromName(MaterialInterface->GetFName()));
				UE_LOGF(LogMeshLODToolset, Warning, "%ls", *WarningText.ToString());
				Progress->AddWarning(WarningText, FProgressCancel::EMessageLevel::UserWarning);
			}
		}
	}


	SourceMaterials.SetNum(Materials.Num());
	TArray<FReadTextureJob> ReadJobs;
	for (int32 mi = 0; mi < Materials.Num(); ++mi)
	{
		SourceMaterials[mi].SourceMaterial = Materials[mi];

		UMaterialInterface* MaterialInterface = Materials[mi].MaterialInterface;
		if (MaterialInterface == nullptr)
		{
			continue;
		}

		// detect hard-coded (non-parameter) texture samples
		{
			UMaterial* Material = Materials[mi].MaterialInterface->GetMaterial();

			// go over the nodes in the material graph looking for texture samples
			const UMaterialGraph* MatGraph = Material->MaterialGraph;

			if (MatGraph == nullptr)	// create a material graph from the material if necessary
			{
				UMaterialGraph* NewMatGraph = CastChecked<UMaterialGraph>(NewObject<UEdGraph>(Material, UMaterialGraph::StaticClass(), NAME_None, RF_Transient));
				NewMatGraph->Material = Material;
				NewMatGraph->RebuildGraph();
				MatGraph = NewMatGraph;
			}

			bool bFoundTextureNonParamExpession = false;
			const TArray<TObjectPtr<class UEdGraphNode>>& Nodes = MatGraph->Nodes;
			for (const TObjectPtr<UEdGraphNode>& Node : Nodes)
			{
				const UMaterialGraphNode* GraphNode = Cast<UMaterialGraphNode>(Node);
				if (GraphNode)
				{
					const UMaterialExpressionTextureSampleParameter* TextureSampleParameterBase = Cast<UMaterialExpressionTextureSampleParameter>(GraphNode->MaterialExpression);
					if (!TextureSampleParameterBase)
					{
						const UMaterialExpressionTextureSample* TextureSampleBase = Cast<UMaterialExpressionTextureSample>(GraphNode->MaterialExpression);
						if (TextureSampleBase)
						{
							// node is UMaterialExpressionTextureSample but not UMaterialExpressionTextureSampleParameter
							bFoundTextureNonParamExpession = true;
							break;
						}
					}

				}
			}
			if (bFoundTextureNonParamExpession)
			{
				FText WarningText = FText::Format(LOCTEXT("NonParameterTextureWarning", "Non-parameter texture sampler detected in input material [{0}]. Output materials may have unexpected behaviour."),
												  FText::FromString(Material->GetName()));
				UE_LOGF(LogMeshLODToolset, Warning, "%ls", *WarningText.ToString());
				if (Progress)
				{
					Progress->AddWarning(WarningText, FProgressCancel::EMessageLevel::UserWarning);
				}
			}
		}

		SourceMaterials[mi].bHasNormalMap = false;
		SourceMaterials[mi].bHasTexturesToBake = false;

		TArray<FMaterialParameterInfo> ParameterInfo;
		TArray<FGuid> ParameterIds;
		MaterialInterface->GetAllTextureParameterInfo(ParameterInfo, ParameterIds);
		for (int32 ti = 0; ti < ParameterInfo.Num(); ++ti)
		{
			FName ParamName = ParameterInfo[ti].Name;

			UTexture* CurTexture = nullptr;
			bool bFound = MaterialInterface->GetTextureParameterValue(
				FMemoryImageMaterialParameterInfo(ParameterInfo[ti]), CurTexture);
			if (ensure(bFound))
			{
				if (Cast<UTexture2D>(CurTexture) != nullptr)
				{
					FTextureInfo TexInfo;
					TexInfo.Texture = Cast<UTexture2D>(CurTexture);
					TexInfo.ParameterName = ParamName;

					TexInfo.bIsNormalMap = TexInfo.Texture->IsNormalMap();
					SourceMaterials[mi].bHasNormalMap |= TexInfo.bIsNormalMap;

					TexInfo.bIsDefaultTexture = UEditorAssetLibrary::GetPathNameForLoadedAsset(TexInfo.Texture).StartsWith(TEXT("/Engine/"));

					TexInfo.bShouldBakeTexture = (TexInfo.bIsNormalMap == false && TexInfo.bIsDefaultTexture == false);
					if (TexInfo.bShouldBakeTexture)
					{
						ReadJobs.Add(FReadTextureJob{ mi, SourceMaterials[mi].SourceTextures.Num() });

						SourceMaterials[mi].bHasTexturesToBake = true;
					}

					SourceMaterials[mi].SourceTextures.Add(TexInfo);
				}
			}
		}

		// if material does not have a normal map parameter or any textures we want to bake, we can just re-use it
		SourceMaterials[mi].bIsReusable = (SourceMaterials[mi].bHasNormalMap == false && SourceMaterials[mi].bHasTexturesToBake == false);
	}


	// If we have hi-res source we can discard any materials that are only used on the previously-generated LOD0.
	// We cannot explicitly tag the materials as being generated so we infer, ie we assume a material was generated
	// if it is only used in LOD0 and not HiRes
	if (bUsingHiResSource)
	{
		// have to wait for SourceMesh to finish converting
		ConvertMesh.Wait();

		const FMeshDescription* LOD0MeshDescription = SourceStaticMesh->GetMeshDescription(0);
		FMeshDescriptionToDynamicMesh GetLOD0Mesh;
		FDynamicMesh3 LOD0Mesh;
		GetLOD0Mesh.Convert(LOD0MeshDescription, LOD0Mesh);
		const FDynamicMeshMaterialAttribute* SourceMaterialIDs = SourceMesh.Attributes()->GetMaterialID();
		const FDynamicMeshMaterialAttribute* LOD0MaterialIDs = LOD0Mesh.Attributes()->GetMaterialID();
		if (ensure(SourceMaterialIDs != nullptr && LOD0MaterialIDs != nullptr))
		{
			int32 NumMaterials = SourceMaterials.Num();
			TArray<bool> IsBaseMaterial;
			IsBaseMaterial.Init(false, NumMaterials);
			TArray<bool> IsLOD0Material;
			IsLOD0Material.Init(false, NumMaterials);

			for (int32 tid : SourceMesh.TriangleIndicesItr())
			{
				int32 MatIdx = SourceMaterialIDs->GetValue(tid);
				if (MatIdx >= 0 && MatIdx < NumMaterials)
				{
					IsBaseMaterial[MatIdx] = true;
				}
			}
			for (int32 tid : LOD0Mesh.TriangleIndicesItr())
			{
				int32 MatIdx = LOD0MaterialIDs->GetValue(tid);
				if (MatIdx >= 0 && MatIdx < NumMaterials)
				{
					IsLOD0Material[MatIdx] = true;
				}
			}

			for (int32 k = 0; k < NumMaterials; ++k)
			{
				if (IsLOD0Material[k] == true && IsBaseMaterial[k] == false)
				{
					SourceMaterials[k].bIsPreviouslyGeneratedMaterial = true;
					SourceMaterials[k].bHasTexturesToBake = false;
					SourceMaterials[k].bIsReusable = false;
				}
			}
		}
	}



	// extract all the texture params
	// TODO: this triggers a checkSlow in the serialization code when it runs async. Find out why. Jira: UETOOL-2985
	//TFuture<void> ReadTextures = Async(GenerateSMLODAsyncExecTarget, [&]()
	//{
	//	ParallelFor(ReadJobs.Num(), [&](int32 ji)
	//	{
	//		FReadTextureJob job = ReadJobs[ji];
	//		FTextureInfo& TexInfo = SourceMaterials[job.MatIndex].SourceTextures[job.TexIndex];
	//		UE::AssetUtils::ReadTexture(TexInfo.Texture, TexInfo.Image);
	//		TexInfo.Dimensions = TexInfo.Image.GetDimensions();
	//	});
	//});

	// single-thread path
	TArray<UTexture2D*> UniqueTextures;
	TArray<TImageBuilder<FVector4f>*> UniqueTextureImages;
	for (FReadTextureJob job : ReadJobs)
	{
		// only read textures that are from materials we are going to possibly bake
		FSourceMaterialInfo& SourceMaterial = SourceMaterials[job.MatIndex];
		if (SourceMaterial.bIsPreviouslyGeneratedMaterial == false && SourceMaterial.bIsReusable == false)
		{
			FTextureInfo& TexInfo = SourceMaterials[job.MatIndex].SourceTextures[job.TexIndex];
			UE::AssetUtils::ReadTexture(TexInfo.Texture, TexInfo.Image);
			TexInfo.Dimensions = TexInfo.Image.GetDimensions();

			UniqueTextures.AddUnique(TexInfo.Texture);
			UniqueTextureImages.Add(&TexInfo.Image);
		}
	}

	//ReadTextures.Wait();


	// Now that we have read textures, detect solid-color textures. We do not
	// have to bake these. These are often used for (eg) default textures, etc.
	TArray<bool> bIsSolidColor;
	bIsSolidColor.Init(false, UniqueTextures.Num());
	ParallelFor(UniqueTextures.Num(), [&](int32 ti)
	{
		bIsSolidColor[ti] = UniqueTextureImages[ti]->IsConstantValue();
	});
	for (FSourceMaterialInfo& SourceMaterial : SourceMaterials)
	{
		if (SourceMaterial.bHasTexturesToBake)
		{
			int StillToBakeCount = 0;
			for (FTextureInfo& TexInfo : SourceMaterial.SourceTextures)
			{
				int32 Index = UniqueTextures.IndexOfByKey(TexInfo.Texture);
				if (Index != INDEX_NONE && bIsSolidColor[Index])
				{
					TexInfo.bShouldBakeTexture = false;
				}
				else
				{
					StillToBakeCount++;
				}
			}
			if (StillToBakeCount == 0)
			{
				// If Material don't have any remaining textures to bake, and it doesn't have a normal map, then we do not have to bake it
				SourceMaterial.bHasTexturesToBake = false;
				SourceMaterial.bIsReusable = (SourceMaterial.bHasNormalMap == false);
			}
		}
	}


	ConvertMesh.Wait();

	FString FullPathWithExtension = UEditorAssetLibrary::GetPathNameForLoadedAsset(SourceStaticMesh);
	SourceAssetPath = FPaths::GetBaseFilename(FullPathWithExtension, false);
	SourceAssetFolder = FPaths::GetPath(SourceAssetPath);
	SourceAssetName = FPaths::GetBaseFilename(FullPathWithExtension, true);

	UpdateDerivedPathName(SourceAssetName, GetDefaultDerivedAssetSuffix());

	InitializeGenerator();

	return true;
}


void UGenerateStaticMeshLODProcess::UpdateDerivedPathName(const FString& NewAssetBaseName, const FString& NewAssetSuffix)
{
	DerivedAssetNameNoSuffix = FPaths::MakeValidFileName(NewAssetBaseName.Replace(TEXT(" "), TEXT("_")));
	if (DerivedAssetNameNoSuffix.Len() == 0)
	{
		DerivedAssetNameNoSuffix = SourceAssetName;
	}

	DerivedSuffix = FPaths::MakeValidFileName(NewAssetSuffix.Replace(TEXT(" "), TEXT("_")));
	if (DerivedSuffix.Len() == 0)
	{
		DerivedSuffix = GetDefaultDerivedAssetSuffix();
	}

	DerivedAssetName = FString::Printf(TEXT("%s%s"), *DerivedAssetNameNoSuffix, *DerivedSuffix);
	DerivedAssetFolder = SourceAssetFolder;
	DerivedAssetPath = FPaths::Combine(DerivedAssetFolder, DerivedAssetName);
}


int UGenerateStaticMeshLODProcess::SelectTextureToBake(const TArray<FTextureInfo>& TextureInfos) const
{
	TArray<int> TextureVotes;
	TextureVotes.Init(0, TextureInfos.Num());

	for (int TextureIndex = 0; TextureIndex < TextureInfos.Num(); ++TextureIndex)
	{
		FTextureInfo Info = TextureInfos[TextureIndex];

		if (!Info.bIsNormalMap)
		{
			++TextureVotes[TextureIndex];
		}

		if (Info.ParameterName == "Diffuse" || Info.ParameterName == "Albedo" || Info.ParameterName == "BaseColor")
		{
			++TextureVotes[TextureIndex];
		}

		if (Info.Dimensions.GetWidth() > 0 && Info.Dimensions.GetHeight() > 0)
		{
			++TextureVotes[TextureIndex];
		}

		UTexture2D* Tex2D = Info.Texture;

		if (Tex2D)
		{
			// Texture uses SRGB
			if (Tex2D->SRGB != 0)
			{
				++TextureVotes[TextureIndex];
			}

#if WITH_EDITORONLY_DATA
			// Texture has multiple channels
			ETextureSourceFormat Format = Tex2D->Source.GetFormat();
			if (Format == TSF_BGRA8 || Format == TSF_BGRE8 || Format == TSF_RGBA16 || Format == TSF_RGBA16F || Format == TSF_RGBA32F)
			{
				++TextureVotes[TextureIndex];
			}
#endif

			// What else? Largest texture? Most layers? Most mipmaps?
		}
	}

	int MaxIndex = -1;
	int MaxVotes = -1;
	for (int TextureIndex = 0; TextureIndex < TextureInfos.Num(); ++TextureIndex)
	{
		if (TextureVotes[TextureIndex] > MaxVotes)
		{
			MaxIndex = TextureIndex;
			MaxVotes = TextureVotes[TextureIndex];
		}
	}

	return MaxIndex;
}



bool UGenerateStaticMeshLODProcess::InitializeGenerator()
{
	Generator = MakeUnique<FGenerateMeshLODGraph>();
	Generator->BuildGraph( & this->SourceMesh );

	// initialize source textures that need to be baked
	SourceTextureToDerivedTexIndex.Reset();
	for (const FSourceMaterialInfo& MatInfo : SourceMaterials)
	{
		if ( MatInfo.bIsPreviouslyGeneratedMaterial == false && MatInfo.bIsReusable == false && MatInfo.bHasTexturesToBake)
		{
			for (const FTextureInfo& TexInfo : MatInfo.SourceTextures)
			{
				if (TexInfo.bShouldBakeTexture &&
					(SourceTextureToDerivedTexIndex.Contains(TexInfo.Texture) == false))
				{
					int32 NewIndex = Generator->AppendTextureBakeNode(TexInfo.Image, TexInfo.Texture->GetName());
					SourceTextureToDerivedTexIndex.Add(TexInfo.Texture, NewIndex);
				}
			}
		}
	}

	//  Add multi-texture bake node	
	TMap<int32, TSafeSharedPtr<UE::Geometry::TImageBuilder<FVector4f>>> SourceMaterialImages;

	// TODO: Relying on MaterialID being the index in the SourceMaterials array is brittle
	for (int32 MaterialID = 0; MaterialID < SourceMaterials.Num(); ++MaterialID)
	{
		FSourceMaterialInfo& MatInfo = SourceMaterials[MaterialID];

		if (MatInfo.bIsPreviouslyGeneratedMaterial == false && MatInfo.bIsReusable == false && MatInfo.bHasTexturesToBake)
		{
			int TexIndex = SelectTextureToBake(MatInfo.SourceTextures);
			if (TexIndex < 0)
			{
				continue;
			}

			FTextureInfo& TextureInfo = MatInfo.SourceTextures[TexIndex];

			MultiTextureParameterName.Add(MaterialID, TextureInfo.ParameterName);

			TSafeSharedPtr<TImageBuilder<FVector4f>> ImagePtr =
				MakeSafeShared<TImageBuilder<FVector4f>>(TextureInfo.Image);
			SourceMaterialImages.Add(MaterialID, ImagePtr);

			TextureInfo.bIsUsedInMultiTextureBaking = true;
		}
	}

	Generator->AppendMultiTextureBakeNode(SourceMaterialImages);


	// initialize source mesh
	Generator->SetSourceMesh(this->SourceMesh);


	// read parameter settings from the graph

	CurrentSettings_Preprocess.FilterGroupLayer = Generator->GetCurrentPreFilterSettings().FilterGroupLayerName;
	CurrentSettings_Preprocess.ThickenAmount = Generator->GetCurrentThickenSettings().ThickenAmount;
	CurrentSettings.MeshGenerator = static_cast<EGenerateStaticMeshLODProcess_MeshGeneratorModes>(Generator->GetCurrentCoreMeshGeneratorMode());

	CurrentSettings.SolidifyVoxelResolution = Generator->GetCurrentSolidifySettings().VoxelResolution;
	CurrentSettings.WindingThreshold = Generator->GetCurrentSolidifySettings().WindingThreshold;
	CurrentSettings.ClosureDistance = Generator->GetCurrentMorphologySettings().Distance;
	CurrentSettings.bPrefilterVertices = Generator->GetCurrentGenerateConvexHullMeshSettings().bPrefilterVertices;
	CurrentSettings.PrefilterGridResolution = Generator->GetCurrentGenerateConvexHullMeshSettings().PrefilterGridResolution;

	CurrentSettings_Simplify.Method = static_cast<EGenerateStaticMeshLODProcess_SimplifyMethod>(Generator->GetCurrentSimplifySettings().TargetType);
	CurrentSettings_Simplify.TargetCount = Generator->GetCurrentSimplifySettings().TargetCount;
	CurrentSettings_Simplify.TargetPercentage = Generator->GetCurrentSimplifySettings().TargetFraction * 100.0f;
	CurrentSettings_Simplify.Tolerance = Generator->GetCurrentSimplifySettings().GeometricTolerance;

	CurrentSettings_Normals.Method = static_cast<EGenerateStaticMeshLODProcess_NormalsMethod>(
		FGenerateStaticMeshLODProcess_NormalsSettings::MapMethodType((int32)Generator->GetCurrentNormalsSettings().NormalsType, false));
	CurrentSettings_Normals.Angle = Generator->GetCurrentNormalsSettings().AngleThresholdDeg;

	CurrentSettings_Texture.BakeResolution = (EGenerateStaticMeshLODBakeResolution)Generator->GetCurrentBakeCacheSettings().Dimensions.X;
	CurrentSettings_Texture.BakeThickness = Generator->GetCurrentBakeCacheSettings().Thickness;

	CurrentSettings_UV.UVMethod = static_cast<EGenerateStaticMeshLODProcess_AutoUVMethod>(Generator->GetCurrentAutoUVSettings().Method);
	CurrentSettings_UV.NumUVAtlasCharts = Generator->GetCurrentAutoUVSettings().UVAtlasNumCharts;
	CurrentSettings_UV.NumInitialPatches = Generator->GetCurrentAutoUVSettings().NumInitialPatches;
	CurrentSettings_UV.MergingThreshold = Generator->GetCurrentAutoUVSettings().MergingThreshold;
	CurrentSettings_UV.MaxAngleDeviation = Generator->GetCurrentAutoUVSettings().MaxAngleDeviationDeg;
	CurrentSettings_UV.PatchBuilder.CurvatureAlignment = Generator->GetCurrentAutoUVSettings().CurvatureAlignment;
	CurrentSettings_UV.PatchBuilder.SmoothingSteps = Generator->GetCurrentAutoUVSettings().SmoothingSteps;
	CurrentSettings_UV.PatchBuilder.SmoothingAlpha = Generator->GetCurrentAutoUVSettings().SmoothingAlpha;

	const UE::GeometryFlow::FGenerateSimpleCollisionSettings& SimpleCollisionSettings = Generator->GetCurrentGenerateSimpleCollisionSettings();
	CurrentSettings_Collision.CollisionType = static_cast<EGenerateStaticMeshLODSimpleCollisionGeometryType>(SimpleCollisionSettings.Type);
	CurrentSettings_Collision.ConvexTriangleCount = SimpleCollisionSettings.ConvexHullSettings.SimplifyToTriangleCount;
	CurrentSettings_Collision.bPrefilterVertices = SimpleCollisionSettings.ConvexHullSettings.bPrefilterVertices;
	CurrentSettings_Collision.PrefilterGridResolution = SimpleCollisionSettings.ConvexHullSettings.PrefilterGridResolution;
	CurrentSettings_Collision.bSimplifyPolygons = SimpleCollisionSettings.SweptHullSettings.bSimplifyPolygons;
	CurrentSettings_Collision.HullTolerance = SimpleCollisionSettings.SweptHullSettings.HullTolerance;
	FMeshSimpleShapeApproximation::EProjectedHullAxisMode RHSMode = static_cast<FMeshSimpleShapeApproximation::EProjectedHullAxisMode>(SimpleCollisionSettings.SweptHullSettings.SweepAxis);
	CurrentSettings_Collision.SweepAxis = static_cast<EGenerateStaticMeshLODProjectedHullAxisMode>(RHSMode);


	return true;
}


void UGenerateStaticMeshLODProcess::UpdateSettings(const FGenerateStaticMeshLODProcessSettings& NewCombinedSettings)
{
	if (NewCombinedSettings.MeshGenerator != CurrentSettings.MeshGenerator)
	{
		Generator->UpdateCoreMeshGeneratorMode(static_cast<FGenerateMeshLODGraph::ECoreMeshGeneratorMode>(NewCombinedSettings.MeshGenerator));
	}

	bool bSharedVoxelResolutionChanged = (NewCombinedSettings.SolidifyVoxelResolution != CurrentSettings.SolidifyVoxelResolution);
	if ( bSharedVoxelResolutionChanged
		|| (NewCombinedSettings.WindingThreshold != CurrentSettings.WindingThreshold))
	{
		UE::GeometryFlow::FMeshSolidifySettings NewSolidifySettings = Generator->GetCurrentSolidifySettings();
		NewSolidifySettings.VoxelResolution = NewCombinedSettings.SolidifyVoxelResolution;
		NewSolidifySettings.WindingThreshold = NewCombinedSettings.WindingThreshold;
		Generator->UpdateSolidifySettings(NewSolidifySettings);
	}


	if ( bSharedVoxelResolutionChanged
		|| (NewCombinedSettings.ClosureDistance != CurrentSettings.ClosureDistance))
	{
		UE::GeometryFlow::FVoxClosureSettings NewClosureSettings = Generator->GetCurrentMorphologySettings();
		NewClosureSettings.VoxelResolution = NewCombinedSettings.SolidifyVoxelResolution;
		NewClosureSettings.Distance = NewCombinedSettings.ClosureDistance;
		Generator->UpdateMorphologySettings(NewClosureSettings);
	}

	if (NewCombinedSettings.bPrefilterVertices != CurrentSettings.bPrefilterVertices ||
		NewCombinedSettings.PrefilterGridResolution != CurrentSettings.PrefilterGridResolution)
	{
		UE::GeometryFlow::FGenerateConvexHullMeshSettings NewConvexHullSettings = Generator->GetCurrentGenerateConvexHullMeshSettings();
		NewConvexHullSettings.bPrefilterVertices = NewCombinedSettings.bPrefilterVertices;
		NewConvexHullSettings.PrefilterGridResolution = NewCombinedSettings.PrefilterGridResolution;
		Generator->UpdateGenerateConvexHullMeshSettings(NewConvexHullSettings);
	}

	CurrentSettings = NewCombinedSettings;
}



void UGenerateStaticMeshLODProcess::UpdatePreprocessSettings(const FGenerateStaticMeshLODProcess_PreprocessSettings& NewCombinedSettings)
{
	if (NewCombinedSettings.FilterGroupLayer != CurrentSettings_Preprocess.FilterGroupLayer)
	{
		FMeshLODGraphPreFilterSettings NewPreFilterSettings = Generator->GetCurrentPreFilterSettings();
		NewPreFilterSettings.FilterGroupLayerName = NewCombinedSettings.FilterGroupLayer;
		Generator->UpdatePreFilterSettings(NewPreFilterSettings);
	}

	if (NewCombinedSettings.ThickenAmount != CurrentSettings_Preprocess.ThickenAmount)
	{
		UE::GeometryFlow::FMeshThickenSettings NewThickenSettings = Generator->GetCurrentThickenSettings();
		NewThickenSettings.ThickenAmount = NewCombinedSettings.ThickenAmount;
		Generator->UpdateThickenSettings(NewThickenSettings);
	}

	if (NewCombinedSettings.ThickenWeightMapName != CurrentSettings_Preprocess.ThickenWeightMapName)
	{
		FIndexedWeightMap1f WeightMap;
		float DefaultValue = 0.0f;
		bool bOK = UE::WeightMaps::GetVertexWeightMap(SourceMeshDescription.Get(), NewCombinedSettings.ThickenWeightMapName, WeightMap, DefaultValue);

		if (bOK)
		{
			Generator->UpdateThickenWeightMap(WeightMap.Values);
		}
		else
		{
			Generator->UpdateThickenWeightMap(TArray<float>());
		}
	}

	CurrentSettings_Preprocess = NewCombinedSettings;
}



void UGenerateStaticMeshLODProcess::UpdateSimplifySettings(const FGenerateStaticMeshLODProcess_SimplifySettings& NewCombinedSettings)
{
	if (NewCombinedSettings.Method != CurrentSettings_Simplify.Method || 
		NewCombinedSettings.TargetCount != CurrentSettings_Simplify.TargetCount ||
		NewCombinedSettings.TargetPercentage != CurrentSettings_Simplify.TargetPercentage ||
		NewCombinedSettings.Tolerance != CurrentSettings_Simplify.Tolerance )
	{
		UE::GeometryFlow::FMeshSimplifySettings NewSimplifySettings = Generator->GetCurrentSimplifySettings();
		NewSimplifySettings.TargetType = ToUEnum(static_cast<UE::GeometryFlow::EMeshSimplifyTargetType>(NewCombinedSettings.Method));
		NewSimplifySettings.TargetCount = NewCombinedSettings.TargetCount;
		NewSimplifySettings.TargetFraction = NewCombinedSettings.TargetPercentage / 100.0f;
		NewSimplifySettings.GeometricTolerance = NewCombinedSettings.Tolerance;
		Generator->UpdateSimplifySettings(NewSimplifySettings);
	}

	CurrentSettings_Simplify = NewCombinedSettings;
}



int32 FGenerateStaticMeshLODProcess_NormalsSettings::MapMethodType(int32 From, bool bProcessToGraph)
{
	if (bProcessToGraph)
	{
		switch (static_cast<EGenerateStaticMeshLODProcess_NormalsMethod>(From))
		{
			default:
			case EGenerateStaticMeshLODProcess_NormalsMethod::FromAngleThreshold: return (int32)UE::GeometryFlow::EComputeNormalsType::FromFaceAngleThreshold;
			case EGenerateStaticMeshLODProcess_NormalsMethod::PerVertex: return (int32)UE::GeometryFlow::EComputeNormalsType::PerVertex;
			case EGenerateStaticMeshLODProcess_NormalsMethod::PerTriangle: return (int32)UE::GeometryFlow::EComputeNormalsType::PerTriangle;
		}
	}
	else
	{
		switch (static_cast<UE::GeometryFlow::EComputeNormalsType>(From))
		{
			default:
			case UE::GeometryFlow::EComputeNormalsType::FromFaceAngleThreshold: return (int32)EGenerateStaticMeshLODProcess_NormalsMethod::FromAngleThreshold;
			case UE::GeometryFlow::EComputeNormalsType::PerVertex: return (int32)EGenerateStaticMeshLODProcess_NormalsMethod::PerVertex;
			case UE::GeometryFlow::EComputeNormalsType::PerTriangle: return (int32)EGenerateStaticMeshLODProcess_NormalsMethod::PerTriangle;
		}
	}
}


void UGenerateStaticMeshLODProcess::UpdateNormalsSettings(const FGenerateStaticMeshLODProcess_NormalsSettings& NewCombinedSettings)
{
	if (NewCombinedSettings.Method != CurrentSettings_Normals.Method || 
		NewCombinedSettings.Angle != CurrentSettings_Normals.Angle )
	{
		UE::GeometryFlow::FMeshNormalsSettings NewNormalsSettings = Generator->GetCurrentNormalsSettings();

		NewNormalsSettings.NormalsType = ToUEnum(static_cast<UE::GeometryFlow::EComputeNormalsType>(
			FGenerateStaticMeshLODProcess_NormalsSettings::MapMethodType((int32)NewCombinedSettings.Method, true)));
		NewNormalsSettings.AngleThresholdDeg = NewCombinedSettings.Angle;

		Generator->UpdateNormalsSettings(NewNormalsSettings);
	}

	CurrentSettings_Normals = NewCombinedSettings;
}



void UGenerateStaticMeshLODProcess::UpdateTextureSettings(const FGenerateStaticMeshLODProcess_TextureSettings& NewCombinedSettings)
{
	if ( (NewCombinedSettings.BakeResolution != CurrentSettings_Texture.BakeResolution) ||
		 (NewCombinedSettings.BakeThickness != CurrentSettings_Texture.BakeThickness) || 
		 (NewCombinedSettings.bCombineTextures != CurrentSettings_Texture.bCombineTextures))
	{
		UE::GeometryFlow::FMeshMakeBakingCacheSettings NewBakeSettings = Generator->GetCurrentBakeCacheSettings();
		NewBakeSettings.Dimensions = FIntPoint((int32)NewCombinedSettings.BakeResolution, (int32)NewCombinedSettings.BakeResolution);
		NewBakeSettings.Thickness = NewCombinedSettings.BakeThickness;
		Generator->UpdateBakeCacheSettings(NewBakeSettings);
	}

	CurrentSettings_Texture = NewCombinedSettings;
}




void UGenerateStaticMeshLODProcess::UpdateUVSettings(const FGenerateStaticMeshLODProcess_UVSettings& NewCombinedSettings)
{
	if (NewCombinedSettings.UVMethod != CurrentSettings_UV.UVMethod
		|| NewCombinedSettings.NumUVAtlasCharts != CurrentSettings_UV.NumUVAtlasCharts
		|| NewCombinedSettings.NumInitialPatches != CurrentSettings_UV.NumInitialPatches
		|| NewCombinedSettings.MergingThreshold != CurrentSettings_UV.MergingThreshold
		|| NewCombinedSettings.MaxAngleDeviation != CurrentSettings_UV.MaxAngleDeviation
		|| NewCombinedSettings.PatchBuilder.CurvatureAlignment != CurrentSettings_UV.PatchBuilder.CurvatureAlignment
		|| NewCombinedSettings.PatchBuilder.SmoothingSteps != CurrentSettings_UV.PatchBuilder.SmoothingSteps
		|| NewCombinedSettings.PatchBuilder.SmoothingAlpha != CurrentSettings_UV.PatchBuilder.SmoothingAlpha  ) 
	{
		UE::GeometryFlow::FMeshAutoGenerateUVsSettings NewAutoUVSettings = Generator->GetCurrentAutoUVSettings();
		NewAutoUVSettings.Method = ToUEnum( (EAutoUVMethod)(int)NewCombinedSettings.UVMethod);
		//NewAutoUVSettings.UVAtlasStretch = 0.5;
		NewAutoUVSettings.UVAtlasNumCharts = NewCombinedSettings.NumUVAtlasCharts;
		//NewAutoUVSettings.XAtlasMaxIterations = 1;
		NewAutoUVSettings.NumInitialPatches = NewCombinedSettings.NumInitialPatches;
		NewAutoUVSettings.CurvatureAlignment = NewCombinedSettings.PatchBuilder.CurvatureAlignment;
		NewAutoUVSettings.MergingThreshold = NewCombinedSettings.MergingThreshold;
		NewAutoUVSettings.MaxAngleDeviationDeg = NewCombinedSettings.MaxAngleDeviation;
		NewAutoUVSettings.SmoothingSteps = NewCombinedSettings.PatchBuilder.SmoothingSteps;
		NewAutoUVSettings.SmoothingAlpha = NewCombinedSettings.PatchBuilder.SmoothingAlpha;
		//NewAutoUVSettings.bAutoPack = false;
		//NewAutoUVSettings.PackingTargetWidth = 512;
		Generator->UpdateAutoUVSettings(NewAutoUVSettings);
	}

	CurrentSettings_UV = NewCombinedSettings;
}




void UGenerateStaticMeshLODProcess::UpdateCollisionSettings(const FGenerateStaticMeshLODProcess_CollisionSettings& NewCombinedSettings)
{
	if (NewCombinedSettings.CollisionGroupLayerName != CurrentSettings_Collision.CollisionGroupLayerName)
	{
		Generator->UpdateCollisionGroupLayerName(NewCombinedSettings.CollisionGroupLayerName);
	}

	if (NewCombinedSettings.ConvexTriangleCount != CurrentSettings_Collision.ConvexTriangleCount ||
		NewCombinedSettings.bPrefilterVertices != CurrentSettings_Collision.bPrefilterVertices ||
		NewCombinedSettings.PrefilterGridResolution != CurrentSettings_Collision.PrefilterGridResolution ||
		NewCombinedSettings.bSimplifyPolygons != CurrentSettings_Collision.bSimplifyPolygons ||
		NewCombinedSettings.HullTolerance != CurrentSettings_Collision.HullTolerance ||
		NewCombinedSettings.SweepAxis != CurrentSettings_Collision.SweepAxis ||
		NewCombinedSettings.CollisionType != CurrentSettings_Collision.CollisionType)
	{
		UE::GeometryFlow::FGenerateSimpleCollisionSettings NewGenCollisionSettings = Generator->GetCurrentGenerateSimpleCollisionSettings();
		NewGenCollisionSettings.Type = static_cast<EGeometryFlow_SimpleCollisionGeometryType>(NewCombinedSettings.CollisionType);
		NewGenCollisionSettings.ConvexHullSettings.SimplifyToTriangleCount = NewCombinedSettings.ConvexTriangleCount;
		NewGenCollisionSettings.ConvexHullSettings.bPrefilterVertices = NewCombinedSettings.bPrefilterVertices;
		NewGenCollisionSettings.ConvexHullSettings.PrefilterGridResolution = NewCombinedSettings.PrefilterGridResolution;
		NewGenCollisionSettings.SweptHullSettings.bSimplifyPolygons = NewCombinedSettings.bSimplifyPolygons;
		NewGenCollisionSettings.SweptHullSettings.HullTolerance = NewCombinedSettings.HullTolerance;
		NewGenCollisionSettings.SweptHullSettings.SweepAxis = static_cast<EGeometryFlow_ProjectedHullAxisMode>(NewCombinedSettings.SweepAxis);
		Generator->UpdateGenerateSimpleCollisionSettings(NewGenCollisionSettings);
	}

	CurrentSettings_Collision = NewCombinedSettings;
}



TArray<UMaterialInterface*> UGenerateStaticMeshLODProcess::GetSourceBakeMaterials() const
{
	TArray<UMaterialInterface*> UniqueMaterials;
	for (const FSourceMaterialInfo& MatInfo : SourceMaterials)
	{
		if (MatInfo.bHasTexturesToBake)
		{
			UniqueMaterials.AddUnique(MatInfo.SourceMaterial.MaterialInterface);
		}
	}
	return UniqueMaterials;
}

void UGenerateStaticMeshLODProcess::UpdateSourceBakeMaterialConstraint(UMaterialInterface* Material, EMaterialBakingConstraint Constraint)
{
	for (FSourceMaterialInfo& MatInfo : SourceMaterials)
	{
		if (MatInfo.SourceMaterial.MaterialInterface == Material)
		{
			MatInfo.Constraint = Constraint;
		}
	}
}


TArray<UTexture2D*> UGenerateStaticMeshLODProcess::GetSourceBakeTextures() const
{
	TArray<UTexture2D*> UniqueTextures;
	for (const FSourceMaterialInfo& MatInfo : SourceMaterials)
	{
		if (MatInfo.bHasTexturesToBake)
		{
			for (const FTextureInfo& TexInfo : MatInfo.SourceTextures)
			{
				if (TexInfo.bShouldBakeTexture)
				{
					UniqueTextures.AddUnique(TexInfo.Texture);
				}
			}
		}
	}
	return UniqueTextures;
}


void UGenerateStaticMeshLODProcess::UpdateSourceBakeTextureConstraint(UTexture2D* Texture, ETextureBakingConstraint Constraint)
{
	for (FSourceMaterialInfo& MatInfo : SourceMaterials)
	{
		if (MatInfo.bHasTexturesToBake)
		{
			for (FTextureInfo& TexInfo : MatInfo.SourceTextures)
			{
				if (TexInfo.Texture == Texture)
				{
					TexInfo.Constraint = Constraint;
				}
			}
		}
	}
}





bool UGenerateStaticMeshLODProcess::ComputeDerivedSourceData(FProgressCancel* Progress)
{
	if (Generator == nullptr)
	{
		return false;
	}

	DerivedTextureImages.Reset();

	Generator->EvaluateResult(
		this->DerivedLODMesh,
		this->DerivedLODMeshTangents,
		this->DerivedCollision,
		this->DerivedNormalMapImage,
		this->DerivedTextureImages,
		this->DerivedMultiTextureBakeImage,
		Progress);

	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	// copy materials
	int32 NumMaterials = SourceMaterials.Num();
	DerivedMaterials.SetNum(NumMaterials);
	for (int32 mi = 0; mi < NumMaterials; ++mi)
	{
		DerivedMaterials[mi].SourceMaterialIndex = mi;
		DerivedMaterials[mi].bUseSourceMaterialDirectly = 
			SourceMaterials[mi].bIsReusable 
			|| SourceMaterials[mi].bIsPreviouslyGeneratedMaterial
			|| (SourceMaterials[mi].Constraint == EMaterialBakingConstraint::UseExistingMaterial);
		
		if (DerivedMaterials[mi].bUseSourceMaterialDirectly)
		{
			DerivedMaterials[mi].DerivedMaterial = SourceMaterials[mi].SourceMaterial;
		}
		else
		{
			// TODO this is a lot of wasted overhead, we do not need to copy images here for example
			DerivedMaterials[mi].DerivedTextures = SourceMaterials[mi].SourceTextures;
		}
	}

	// update texture image data in derived materials
	for (FDerivedMaterialInfo& MatInfo : DerivedMaterials)
	{
		for (FTextureInfo& TexInfo : MatInfo.DerivedTextures)
		{
			if (SourceTextureToDerivedTexIndex.Contains(TexInfo.Texture))
			{
				int32 BakedTexIndex = SourceTextureToDerivedTexIndex[TexInfo.Texture];
				const TUniquePtr<UE::GeometryFlow::FTextureImage>& DerivedTex = DerivedTextureImages[BakedTexIndex];
				TexInfo.Dimensions = DerivedTex->Image.GetDimensions();

				// Cannot currently MoveTemp here because this Texture may appear in multiple Materials, 
				// and currently we do not handle that. The Materials need to learn how to share.
				//TexInfo.Image = MoveTemp(DerivedTex->Image);
				TexInfo.Image = DerivedTex->Image;
			}
		}
	}

	return true;
}




void UGenerateStaticMeshLODProcess::GetDerivedMaterialsPreview(FPreviewMaterials& MaterialSetOut)
{
	// force garbage collection of outstanding preview materials
	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

	bool bAnyMaterialUsedGeneratedNormalMap = false;

	// create derived textures
	int32 NumMaterials = SourceMaterials.Num();
	check(DerivedMaterials.Num() == NumMaterials);
	TMap<UTexture2D*, UTexture2D*> SourceToPreviewTexMap;
	for (int32 mi = 0; mi < NumMaterials; ++mi)
	{
		const FSourceMaterialInfo& SourceMaterialInfo = SourceMaterials[mi];
		if (SourceMaterialInfo.bIsReusable 
			|| SourceMaterialInfo.bIsPreviouslyGeneratedMaterial
			|| SourceMaterialInfo.Constraint == EMaterialBakingConstraint::UseExistingMaterial )
		{
			continue;
		}

		if (SourceMaterialInfo.bHasNormalMap)
		{
			bAnyMaterialUsedGeneratedNormalMap = true;
		}

		const FDerivedMaterialInfo& DerivedMaterialInfo = DerivedMaterials[mi];

		int32 NumTextures = SourceMaterialInfo.SourceTextures.Num();
		check(DerivedMaterialInfo.DerivedTextures.Num() == NumTextures);
		for (int32 ti = 0; ti < NumTextures; ++ti)
		{
			const FTextureInfo& SourceTex = SourceMaterialInfo.SourceTextures[ti];

			if (CurrentSettings_Texture.bCombineTextures && SourceTex.bIsUsedInMultiTextureBaking)
			{
				continue;
			}

			bool bConvertToSRGB = SourceTex.Texture->SRGB;
			const FTextureInfo& DerivedTex = DerivedMaterialInfo.DerivedTextures[ti];
			if (DerivedTex.bShouldBakeTexture
				&& SourceTex.Constraint != ETextureBakingConstraint::UseExistingTexture )
			{
				FTexture2DBuilder TextureBuilder;
				TextureBuilder.Initialize(FTexture2DBuilder::ETextureType::Color, DerivedTex.Dimensions);
				TextureBuilder.GetTexture2D()->SRGB = bConvertToSRGB;
				TextureBuilder.Copy(DerivedTex.Image, bConvertToSRGB);
				TextureBuilder.Commit(false);
				UTexture2D* PreviewTex = TextureBuilder.GetTexture2D();
				if (ensure(PreviewTex))
				{
					SourceToPreviewTexMap.Add(SourceTex.Texture, PreviewTex);
					MaterialSetOut.Textures.Add(PreviewTex);
				}
			}
		}
	}

	// create derived normal map texture
	UTexture2D* PreviewNormalMapTex = nullptr;
	if (bAnyMaterialUsedGeneratedNormalMap)
	{
		FTexture2DBuilder NormapMapBuilder;
		NormapMapBuilder.Initialize(FTexture2DBuilder::ETextureType::NormalMap, DerivedNormalMapImage.Image.GetDimensions());
		NormapMapBuilder.Copy(DerivedNormalMapImage.Image, false);
		NormapMapBuilder.Commit(false);
		PreviewNormalMapTex = NormapMapBuilder.GetTexture2D();
		MaterialSetOut.Textures.Add(PreviewNormalMapTex);
	}

	// create multi-texture bake result
	if (CurrentSettings_Texture.bCombineTextures)
	{
		bool bConvertToSRGB = true;

		FTexture2DBuilder MultiTextureBuilder;
		MultiTextureBuilder.Initialize(FTexture2DBuilder::ETextureType::Color, DerivedMultiTextureBakeImage.Image.GetDimensions());
		MultiTextureBuilder.GetTexture2D()->SRGB = bConvertToSRGB;
		MultiTextureBuilder.Copy(DerivedMultiTextureBakeImage.Image, bConvertToSRGB);
		MultiTextureBuilder.Commit(false);

		DerivedMultiTextureBakeResult = MultiTextureBuilder.GetTexture2D();
		MaterialSetOut.Textures.Add(DerivedMultiTextureBakeResult);
	}


	// create derived MIDs and point to new textures
	for (int32 mi = 0; mi < NumMaterials; ++mi)
	{
		const FSourceMaterialInfo& SourceMaterialInfo = SourceMaterials[mi];
		UMaterialInterface* MaterialInterface = SourceMaterialInfo.SourceMaterial.MaterialInterface;

		if (SourceMaterialInfo.bIsReusable 
			|| SourceMaterialInfo.bIsPreviouslyGeneratedMaterial
			|| SourceMaterialInfo.Constraint == EMaterialBakingConstraint::UseExistingMaterial )
		{
			MaterialSetOut.Materials.Add(MaterialInterface);
		}
		else
		{
			// TODO: we should cache these instead of re-creating every time??
			UMaterialInstanceDynamic* GeneratedMID = UMaterialInstanceDynamic::Create(MaterialInterface, NULL);

			// rewrite texture parameters to new textures
			UpdateMaterialTextureParameters(GeneratedMID, SourceMaterialInfo, SourceToPreviewTexMap, PreviewNormalMapTex);

			if (CurrentSettings_Texture.bCombineTextures && DerivedMultiTextureBakeResult && MultiTextureParameterName.Contains(mi))
			{
				FMaterialParameterInfo ParamInfo(MultiTextureParameterName[mi]);
				GeneratedMID->SetTextureParameterValueByInfo(ParamInfo, DerivedMultiTextureBakeResult);
			}

			MaterialSetOut.Materials.Add(GeneratedMID);
		}

	}

}


void UGenerateStaticMeshLODProcess::UpdateMaterialTextureParameters(
	UMaterialInstanceDynamic* Material, 
	const FSourceMaterialInfo& SourceMaterialInfo,
	const TMap<UTexture2D*, UTexture2D*>& PreviewTextures, 
	UTexture2D* PreviewNormalMap)
{
	ensure((SourceMaterialInfo.bHasNormalMap == false) || (PreviewNormalMap != nullptr));

	Material->Modify();
	int32 NumTextures = SourceMaterialInfo.SourceTextures.Num();
	for (int32 ti = 0; ti < NumTextures; ++ti)
	{
		const FTextureInfo& SourceTex = SourceMaterialInfo.SourceTextures[ti];

		if (SourceTex.Constraint == ETextureBakingConstraint::UseExistingTexture)
		{
			FMaterialParameterInfo ParamInfo(SourceTex.ParameterName);
			Material->SetTextureParameterValueByInfo(ParamInfo, SourceTex.Texture);
		}
		else if (SourceTex.bIsNormalMap)
		{
			if (PreviewNormalMap)
			{
				FMaterialParameterInfo ParamInfo(SourceTex.ParameterName);
				Material->SetTextureParameterValueByInfo(ParamInfo, PreviewNormalMap);
			}
		}
		else if (SourceTex.bShouldBakeTexture)
		{
			if (CurrentSettings_Texture.bCombineTextures && SourceTex.bIsUsedInMultiTextureBaking)
			{
				continue;
			}

			UTexture2D*const* FoundTexture = PreviewTextures.Find(SourceTex.Texture);
			if (ensure(FoundTexture))
			{
				FMaterialParameterInfo ParamInfo(SourceTex.ParameterName);
				Material->SetTextureParameterValueByInfo(ParamInfo, *FoundTexture);
			}
		}
	}
	Material->PostEditChange();
}





bool UGenerateStaticMeshLODProcess::WriteDerivedAssetData()
{
	AllDerivedTextures.Reset();

	constexpr bool bCreatingNewStaticMeshAsset = true;
	DerivedAssetGUIDKey = FGuid::NewGuid().ToString(EGuidFormats::UniqueObjectGuid).ToUpper();

	WriteDerivedTextures(bCreatingNewStaticMeshAsset);

	WriteDerivedMaterials(bCreatingNewStaticMeshAsset);

	WriteDerivedStaticMeshAsset();

	// clear list of derived textures we were holding onto to prevent GC
	AllDerivedTextures.Reset();

	return true;
}


void UGenerateStaticMeshLODProcess::UpdateSourceAsset(bool bSetNewHDSourceAsset)
{
	AllDerivedTextures.Reset();

	constexpr bool bCreatingNewStaticMeshAsset = false;
	DerivedAssetGUIDKey = FGuid::NewGuid().ToString(EGuidFormats::UniqueObjectGuid).ToUpper();

	WriteDerivedTextures(bCreatingNewStaticMeshAsset);

	WriteDerivedMaterials(bCreatingNewStaticMeshAsset);

	UpdateSourceStaticMeshAsset(bSetNewHDSourceAsset);

	// clear list of derived textures we were holding onto to prevent GC
	AllDerivedTextures.Reset();
}



bool UGenerateStaticMeshLODProcess::IsSourceAsset(const FString& AssetPath) const
{
	if (UEditorAssetLibrary::DoesAssetExist(AssetPath))
	{
		const FAssetData AssetData = UEditorAssetLibrary::FindAssetData(AssetPath);

		for (const FSourceMaterialInfo& MaterialInfo : SourceMaterials)
		{
			UMaterialInterface* MaterialInterface = MaterialInfo.SourceMaterial.MaterialInterface;
			if (MaterialInterface == nullptr)
			{
				continue;
			}

			FString SourceMaterialPath = UEditorAssetLibrary::GetPathNameForLoadedAsset(MaterialInterface);
			if (UEditorAssetLibrary::FindAssetData(SourceMaterialPath) == AssetData)
			{
				return true;
			}

			for (const FTextureInfo& TextureInfo : MaterialInfo.SourceTextures)
			{
				FString SourceTexturePath = UEditorAssetLibrary::GetPathNameForLoadedAsset(TextureInfo.Texture);
				if (UEditorAssetLibrary::FindAssetData(SourceTexturePath) == AssetData)
				{
					return true;
				}
			}
		}
	}

	return false;
}



void UGenerateStaticMeshLODProcess::WriteDerivedTextures(bool bCreatingNewStaticMeshAsset)
{
	IAssetTools& AssetTools = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

	// this is a workaround for handling multiple materials that reference the same texture. Currently the code
	// below will try to write that texture multiple times, which will fail when it tries to create a package
	// for a filename that already exists
	TMap<UTexture2D*, UTexture2D*> WrittenSourceToDerived;

	// figure out which textures we might write have actually been referenced by a
	// Material we are going to write. We do not want to write unreferenced textures.
	bool bAnyMaterialUsedGeneratedNormalMap = false;
	bool bAnyMaterialUsedMultiTexture = false;
	int32 NumMaterials = SourceMaterials.Num();
	for (int32 mi = 0; mi < NumMaterials; ++mi)
	{
		const FSourceMaterialInfo& SourceMaterialInfo = SourceMaterials[mi];
		if (SourceMaterialInfo.bIsReusable
			|| SourceMaterialInfo.bIsPreviouslyGeneratedMaterial
			|| SourceMaterialInfo.Constraint == EMaterialBakingConstraint::UseExistingMaterial)
		{
			continue;
		}
		if (SourceMaterialInfo.bHasNormalMap)
		{
			bAnyMaterialUsedGeneratedNormalMap = true;
		}
		if (CurrentSettings_Texture.bCombineTextures && MultiTextureParameterName.Contains(mi))
		{
			bAnyMaterialUsedMultiTexture = true;
		}
	}


	// write derived textures
	check(DerivedMaterials.Num() == NumMaterials);
	for (int32 mi = 0; mi < NumMaterials; ++mi)
	{
		const FSourceMaterialInfo& SourceMaterialInfo = SourceMaterials[mi];
		if (SourceMaterialInfo.bIsReusable 
			|| SourceMaterialInfo.bIsPreviouslyGeneratedMaterial
			|| SourceMaterialInfo.Constraint == EMaterialBakingConstraint::UseExistingMaterial )
		{
			continue;
		}

		FDerivedMaterialInfo& DerivedMaterialInfo = DerivedMaterials[mi];

		int32 NumTextures = SourceMaterialInfo.SourceTextures.Num();
		check(DerivedMaterialInfo.DerivedTextures.Num() == NumTextures);
		for (int32 ti = 0; ti < NumTextures; ++ti)
		{
			const FTextureInfo& SourceTex = SourceMaterialInfo.SourceTextures[ti];
			FTextureInfo& DerivedTex = DerivedMaterialInfo.DerivedTextures[ti];

			if (WrittenSourceToDerived.Contains(SourceTex.Texture))
			{
				// Already computed the derived texture from this source
				DerivedTex.Texture = WrittenSourceToDerived[SourceTex.Texture];
				continue;
			}

			bool bConvertToSRGB = SourceTex.Texture->SRGB;

			if (DerivedTex.bShouldBakeTexture == false ||
				DerivedTex.Constraint == ETextureBakingConstraint::UseExistingTexture)
			{
				continue;
			}

			FTexture2DBuilder TextureBuilder;
			TextureBuilder.Initialize(FTexture2DBuilder::ETextureType::Color, DerivedTex.Dimensions);
			TextureBuilder.GetTexture2D()->SRGB = bConvertToSRGB;
			TextureBuilder.Copy(DerivedTex.Image, bConvertToSRGB);
			TextureBuilder.Commit(false);

			DerivedTex.Texture = TextureBuilder.GetTexture2D();
			if (ensure(DerivedTex.Texture))
			{
				AllDerivedTextures.Add(DerivedTex.Texture);

				FTexture2DBuilder::CopyPlatformDataToSourceData(DerivedTex.Texture, FTexture2DBuilder::ETextureType::Color);

				// write asset
				bool bWriteOK = WriteDerivedTexture(SourceTex.Texture, DerivedTex.Texture, bCreatingNewStaticMeshAsset);
				ensure(bWriteOK);

				WrittenSourceToDerived.Add(SourceTex.Texture, DerivedTex.Texture);
			}
		}

	}


	// write derived normal map
	if (bAnyMaterialUsedGeneratedNormalMap) 
	{
		FTexture2DBuilder NormapMapBuilder;
		NormapMapBuilder.Initialize(FTexture2DBuilder::ETextureType::NormalMap, DerivedNormalMapImage.Image.GetDimensions());
		NormapMapBuilder.Copy(DerivedNormalMapImage.Image, false);
		NormapMapBuilder.Commit(false);

		DerivedNormalMapTex = NormapMapBuilder.GetTexture2D();
		if (ensure(DerivedNormalMapTex))
		{
			FTexture2DBuilder::CopyPlatformDataToSourceData(DerivedNormalMapTex, FTexture2DBuilder::ETextureType::NormalMap);

			// write asset
			bool bWriteOK = WriteDerivedTexture(DerivedNormalMapTex, DerivedAssetNameNoSuffix + TEXT("_NormalMap"), bCreatingNewStaticMeshAsset);
			ensure(bWriteOK);
		}
	}


	// write multi-texture bake result
	if (CurrentSettings_Texture.bCombineTextures && bAnyMaterialUsedMultiTexture)
	{
		bool bConvertToSRGB = true;
		
		FTexture2DBuilder MultiTextureBuilder;
		MultiTextureBuilder.Initialize(FTexture2DBuilder::ETextureType::Color, DerivedMultiTextureBakeImage.Image.GetDimensions());
		MultiTextureBuilder.GetTexture2D()->SRGB = bConvertToSRGB;
		MultiTextureBuilder.Copy(DerivedMultiTextureBakeImage.Image, bConvertToSRGB);
		MultiTextureBuilder.Commit(false);

		DerivedMultiTextureBakeResult = MultiTextureBuilder.GetTexture2D();
		if (ensure(DerivedMultiTextureBakeResult))
		{
			FTexture2DBuilder::CopyPlatformDataToSourceData(DerivedMultiTextureBakeResult, FTexture2DBuilder::ETextureType::Color);

			// write asset
			bool bWriteOK = WriteDerivedTexture(DerivedMultiTextureBakeResult, DerivedAssetNameNoSuffix + TEXT("_MultiTexture"), bCreatingNewStaticMeshAsset);
			ensure(bWriteOK);
		}
	}
	
}



bool UGenerateStaticMeshLODProcess::WriteDerivedTexture(UTexture2D* SourceTexture, UTexture2D* DerivedTexture, bool bCreatingNewStaticMeshAsset)
{
	IAssetTools& AssetTools = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

	FString SourceTexPath = UEditorAssetLibrary::GetPathNameForLoadedAsset(SourceTexture);
	FString TexName = FPaths::GetBaseFilename(SourceTexPath, true);
	return WriteDerivedTexture(DerivedTexture, TexName, bCreatingNewStaticMeshAsset);
}



bool UGenerateStaticMeshLODProcess::WriteDerivedTexture(UTexture2D* DerivedTexture, FString BaseTexName, bool bCreatingNewStaticMeshAsset)
{
	IAssetTools& AssetTools = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

	FString NewTexName = FString::Printf(TEXT("%s%s"), *BaseTexName, *DerivedSuffix);
	FString NewAssetPath = FPaths::Combine(DerivedAssetFolder, NewTexName);

	bool bNewAssetExistsInMemory = IsSourceAsset(NewAssetPath);

	if (bCreatingNewStaticMeshAsset || bNewAssetExistsInMemory)
	{
		// Don't delete an existing asset. If name collision occurs, rename the new asset.
		bool bNewAssetExists = UEditorAssetLibrary::DoesAssetExist(NewAssetPath);
		while (bNewAssetExists)
		{
			GenerateStaticMeshLODProcessHelpers::AppendOrIncrementSuffix(NewTexName);
			NewAssetPath = FPaths::Combine(DerivedAssetFolder, NewTexName);
			bNewAssetExists = UEditorAssetLibrary::DoesAssetExist(NewAssetPath);
		}
	}
	else
	{
		// Modifying the static mesh in place. Delete existing asset so that we can have a clean duplicate
		bool bNewAssetExists = UEditorAssetLibrary::DoesAssetExist(NewAssetPath);
		if (bNewAssetExists)
		{
			bool bDeleteOK = UEditorAssetLibrary::DeleteAsset(NewAssetPath);
			ensure(bDeleteOK);
		}
	}

	// create package
	FString UniquePackageName, UniqueAssetName;
	AssetTools.CreateUniqueAssetName(NewAssetPath, TEXT(""), UniquePackageName, UniqueAssetName);
	UPackage* AssetPackage = CreatePackage(*UniquePackageName);
	check(AssetPackage);

	// move texture from Transient package to new package
	DerivedTexture->Rename(*UniqueAssetName, AssetPackage, REN_None);
	// remove transient flag, add public/standalone/transactional
	DerivedTexture->ClearFlags(RF_Transient);
	DerivedTexture->SetFlags(RF_Public | RF_Standalone | RF_Transactional);
	// do we need to Modify() it? we are not doing any undo/redo
	DerivedTexture->Modify();

	// set metadata tag so we can identify this material later
	DerivedTexture->GetPackage()->GetMetaData().SetValue(DerivedTexture, TEXT("StaticMeshLOD.IsBakedTexture"), TEXT("true"));
	DerivedTexture->GetPackage()->GetMetaData().SetValue(DerivedTexture, TEXT("StaticMeshLOD.SourceAssetPath"), *GetSourceAssetPath());
	DerivedTexture->GetPackage()->GetMetaData().SetValue(DerivedTexture, TEXT("StaticMeshLOD.GenerationGUID"), *DerivedAssetGUIDKey);

	DerivedTexture->UpdateResource();
	DerivedTexture->PostEditChange();		// this may be necessary if any Materials are using this texture
	DerivedTexture->MarkPackageDirty();

	FAssetRegistryModule::AssetCreated(DerivedTexture);		// necessary?

	return true;
}


void UGenerateStaticMeshLODProcess::WriteDerivedMaterials(bool bCreatingNewStaticMeshAsset)
{
	IAssetTools& AssetTools = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

	int32 NumMaterials = SourceMaterials.Num();
	check(DerivedMaterials.Num() == NumMaterials);
	for (int32 mi = 0; mi < NumMaterials; ++mi)
	{
		const FSourceMaterialInfo& SourceMaterialInfo = SourceMaterials[mi];
		if (SourceMaterialInfo.bIsReusable 
			|| SourceMaterialInfo.bIsPreviouslyGeneratedMaterial
			|| SourceMaterialInfo.Constraint == EMaterialBakingConstraint::UseExistingMaterial )
		{
			continue;
		}

		FDerivedMaterialInfo& DerivedMaterialInfo = DerivedMaterials[mi];

		UMaterialInterface* MaterialInterface = SourceMaterialInfo.SourceMaterial.MaterialInterface;
		if (MaterialInterface == nullptr)
		{
			DerivedMaterialInfo.DerivedMaterial.MaterialInterface = nullptr;
			continue;
		}
		bool bSourceIsMIC = (Cast<UMaterialInstanceConstant>(MaterialInterface) != nullptr);

		FString SourceMaterialPath = UEditorAssetLibrary::GetPathNameForLoadedAsset(MaterialInterface);
		FString MaterialName = FPaths::GetBaseFilename(SourceMaterialPath, true);
		FString NewMaterialName = FString::Printf(TEXT("%s%s"), *MaterialName, *DerivedSuffix);
		FString NewMaterialPath = FPaths::Combine(DerivedAssetFolder, NewMaterialName);
		bool bNewAssetExistsInMemory = IsSourceAsset(NewMaterialPath);

		if (bCreatingNewStaticMeshAsset || bNewAssetExistsInMemory)
		{
			// Don't delete an existing material. If name collision occurs, rename the new material.
			bool bNewAssetExists = UEditorAssetLibrary::DoesAssetExist(NewMaterialPath);
			while (bNewAssetExists)
			{
				GenerateStaticMeshLODProcessHelpers::AppendOrIncrementSuffix(NewMaterialName);
				NewMaterialPath = FPaths::Combine(DerivedAssetFolder, NewMaterialName);
				bNewAssetExists = UEditorAssetLibrary::DoesAssetExist(NewMaterialPath);
			}
		}
		else
		{
			// Modifying the static mesh in place. Delete existing asset so that we can have a clean duplicate
			bool bNewAssetExists = UEditorAssetLibrary::DoesAssetExist(NewMaterialPath);
			if (bNewAssetExists)
			{
				bool bDeleteOK = UEditorAssetLibrary::DeleteAsset(NewMaterialPath);
				ensure(bDeleteOK);
			}
		}

		// If source is a MIC, we can just duplicate it. If it is a UMaterial, we want to
		// create a child MIC? Or we could dupe the Material and rewrite the textures.
		// Probably needs to be an option.
		UMaterialInstanceConstant* GeneratedMIC = nullptr;
		if (bSourceIsMIC)
		{
			UObject* DupeAsset = UEditorAssetLibrary::DuplicateAsset(SourceMaterialPath, NewMaterialPath);
			GeneratedMIC = Cast<UMaterialInstanceConstant>(DupeAsset);
		}
		else
		{
			UMaterial* SourceMaterial = MaterialInterface->GetBaseMaterial();
			if (ensure(SourceMaterial))
			{
				UMaterialInstanceConstantFactoryNew* Factory = NewObject<UMaterialInstanceConstantFactoryNew>();
				Factory->InitialParent = SourceMaterial;

				UObject* NewAsset = AssetTools.CreateAsset(NewMaterialName, FPackageName::GetLongPackagePath(NewMaterialPath),
					UMaterialInstanceConstant::StaticClass(), Factory);

				GeneratedMIC = Cast<UMaterialInstanceConstant>(NewAsset);
			}
		}

		if (GeneratedMIC != nullptr)
		{
			// set metadata tag so we can identify this material later
			GeneratedMIC->GetPackage()->GetMetaData().SetValue(GeneratedMIC, TEXT("StaticMeshLOD.IsGeneratedMaterial"), TEXT("true"));
			GeneratedMIC->GetPackage()->GetMetaData().SetValue(GeneratedMIC, TEXT("StaticMeshLOD.SourceAssetPath"), *GetSourceAssetPath());
			GeneratedMIC->GetPackage()->GetMetaData().SetValue(GeneratedMIC, TEXT("StaticMeshLOD.GenerationGUID"), *DerivedAssetGUIDKey);

			// rewrite texture parameters to new textures
			UpdateMaterialTextureParameters(GeneratedMIC, DerivedMaterialInfo);
		}

		// update StaticMaterial
		DerivedMaterialInfo.DerivedMaterial.MaterialInterface = GeneratedMIC;
		DerivedMaterialInfo.DerivedMaterial.MaterialSlotName = FName(FString::Printf(TEXT("GeneratedMat%d"), mi));
		DerivedMaterialInfo.DerivedMaterial.ImportedMaterialSlotName = DerivedMaterialInfo.DerivedMaterial.MaterialSlotName;

		if (GeneratedMIC != nullptr)
		{
			if (CurrentSettings_Texture.bCombineTextures && DerivedMultiTextureBakeResult && MultiTextureParameterName.Contains(mi))
			{
				FMaterialParameterInfo ParamInfo(MultiTextureParameterName[mi]);
				GeneratedMIC->SetTextureParameterValueEditorOnly(ParamInfo, DerivedMultiTextureBakeResult);
			}
		}

	}
}



void UGenerateStaticMeshLODProcess::UpdateMaterialTextureParameters(UMaterialInstanceConstant* Material, FDerivedMaterialInfo& DerivedMaterialInfo)
{
	Material->Modify();

	int32 NumTextures = DerivedMaterialInfo.DerivedTextures.Num();
	for (int32 ti = 0; ti < NumTextures; ++ti)
	{
		FTextureInfo& DerivedTex = DerivedMaterialInfo.DerivedTextures[ti];

		if (DerivedTex.Constraint == ETextureBakingConstraint::UseExistingTexture)
		{
			UTexture2D* SourceTexture = SourceMaterials[DerivedMaterialInfo.SourceMaterialIndex].SourceTextures[ti].Texture;
			FMaterialParameterInfo ParamInfo(DerivedTex.ParameterName);
			Material->SetTextureParameterValueEditorOnly(ParamInfo, SourceTexture);
		}
		else if (DerivedTex.bIsNormalMap)
		{
			if (ensure(DerivedNormalMapTex))
			{
				FMaterialParameterInfo ParamInfo(DerivedTex.ParameterName);
				Material->SetTextureParameterValueEditorOnly(ParamInfo, DerivedNormalMapTex);
			}
		}
		else if (DerivedTex.bShouldBakeTexture)
		{
			UTexture2D* NewTexture = DerivedTex.Texture;
			if (ensure(NewTexture))
			{
				FMaterialParameterInfo ParamInfo(DerivedTex.ParameterName);
				Material->SetTextureParameterValueEditorOnly(ParamInfo, NewTexture);
			}
		}
	}

	Material->PostEditChange();
}



void UGenerateStaticMeshLODProcess::WriteDerivedStaticMeshAsset()
{
	// [TODO] should we try to re-use existing asset here, or should we delete it? 
	// The source asset might have had any number of config changes that we want to
	// preserve in the duplicate...
	UStaticMesh* GeneratedStaticMesh = nullptr;
	if (UEditorAssetLibrary::DoesAssetExist(DerivedAssetPath))
	{
		UObject* LoadedAsset = UEditorAssetLibrary::LoadAsset(DerivedAssetPath);
		GeneratedStaticMesh = Cast<UStaticMesh>(LoadedAsset);
	}
	else
	{
		UObject* DupeAsset = UEditorAssetLibrary::DuplicateAsset(SourceAssetPath, DerivedAssetPath);
		GeneratedStaticMesh = Cast<UStaticMesh>(DupeAsset);
	}

	// DuplicateAsset can fail in the event of a dead package from an Asset Reload.
	// This edge case is currently prevented by the GenerateStaticMeshLODAssetToolBuilder.
	// [TODO] Handle a failed DuplicateAsset case by creating a new StaticMesh asset.
	if (!ensure(GeneratedStaticMesh))
	{
		return;
	}

	// make sure transactional flag is on
	GeneratedStaticMesh->SetFlags(RF_Transactional);
	GeneratedStaticMesh->Modify();

	// update MeshDescription LOD0 mesh
	GeneratedStaticMesh->SetNumSourceModels(1);
	FMeshDescription* MeshDescription = GeneratedStaticMesh->GetMeshDescription(0);
	FConversionToMeshDescriptionOptions ConversionOptions;
	FDynamicMeshToMeshDescription Converter(ConversionOptions);
	Converter.Convert(&DerivedLODMesh, *MeshDescription);
	GeneratedStaticMesh->CommitMeshDescription(0);

	// construct new material slots list
	TArray<FStaticMaterial> NewMaterials;
	int32 NumMaterials = SourceMaterials.Num();
	for (int32 mi = 0; mi < NumMaterials; ++mi)
	{
		if (!SourceMaterials[mi].bIsPreviouslyGeneratedMaterial)	// Skip previously generated
		{
			if (SourceMaterials[mi].bIsReusable || SourceMaterials[mi].Constraint == EMaterialBakingConstraint::UseExistingMaterial )
			{
				NewMaterials.Add(SourceMaterials[mi].SourceMaterial);
			}
			else
			{
				NewMaterials.Add(DerivedMaterials[mi].DerivedMaterial);
			}
		}
	}

	// update materials on generated mesh
	GeneratedStaticMesh->SetStaticMaterials(NewMaterials);

	// collision
	FPhysicsDataCollection NewCollisionGeo;
	NewCollisionGeo.Geometry = DerivedCollision;
	NewCollisionGeo.CopyGeometryToAggregate();

	// code below derived from FStaticMeshEditor::DuplicateSelectedPrims()
	UBodySetup* BodySetup = GeneratedStaticMesh->GetBodySetup();
	// mark the BodySetup for modification. Do we need to modify the UStaticMesh??
	BodySetup->Modify();
	//Clear the cache (PIE may have created some data), create new GUID    (comment from StaticMeshEditor)
	BodySetup->InvalidatePhysicsData();
	BodySetup->RemoveSimpleCollision();
	BodySetup->AggGeom = NewCollisionGeo.AggGeom;
	// update collision type
	BodySetup->CollisionTraceFlag = ECollisionTraceFlag::CTF_UseDefault;
	// rebuild physics data
	BodySetup->InvalidatePhysicsData();
	BodySetup->CreatePhysicsMeshes();

	// do we need to do a post edit change here??

	GeneratedStaticMesh->RecreateNavCollision();

	GeneratedStaticMesh->GetPackage()->GetMetaData().SetValue(GeneratedStaticMesh, TEXT("StaticMeshLOD.IsGeneratedMesh"), TEXT("true"));
	GeneratedStaticMesh->GetPackage()->GetMetaData().SetValue(GeneratedStaticMesh, TEXT("StaticMeshLOD.SourceAssetPath"), *GetSourceAssetPath());
	GeneratedStaticMesh->GetPackage()->GetMetaData().SetValue(GeneratedStaticMesh, TEXT("StaticMeshLOD.GenerationGUID"), *DerivedAssetGUIDKey);

	// done updating mesh
	GeneratedStaticMesh->PostEditChange();
}



void UGenerateStaticMeshLODProcess::UpdateSourceStaticMeshAsset(bool bSetNewHDSourceAsset)
{
	GEditor->BeginTransaction(LOCTEXT("UpdateExistingAssetMessage", "Added Generated LOD"));

	// make sure transactional flag is on for this asset
	SourceStaticMesh->SetFlags(RF_Transactional);
	SourceStaticMesh->Modify();

	FStaticMeshSourceModel& SrcModel = SourceStaticMesh->GetSourceModel(0);
	SourceStaticMesh->ModifyMeshDescription(0);

	// if we want to save the input high-poly asset as the HiResSource, do that here
	if (bSetNewHDSourceAsset && bUsingHiResSource == false)
	{
		SourceStaticMesh->ModifyHiResMeshDescription();

		FMeshDescription* NewSourceMD = SourceStaticMesh->CreateHiResMeshDescription();
		*NewSourceMD = *SourceMeshDescription;		// todo: can MoveTemp here, we don't need this memory anymore??

		FStaticMeshSourceModel& HiResSrcModel = SourceStaticMesh->GetHiResSourceModel();
		// Generally copy LOD0 build settings, although many of these will be ignored
		HiResSrcModel.BuildSettings = SrcModel.BuildSettings;
		// on the HiRes we store the existing normals and tangents, which we already auto-computed if necessary
		HiResSrcModel.BuildSettings.bRecomputeNormals = false;
		HiResSrcModel.BuildSettings.bRecomputeTangents = false;
		// TODO: what should we do about Lightmap UVs?

		SourceStaticMesh->CommitHiResMeshDescription();
	}

	// Next bit is tricky, we have to build the final StaticMesh Material Set.
	// We have the existing Source materials we want to keep, except if some
	// were identified as being auto-generated by a previous run of AutoLOD, we want to leave those out.
	// Then we want to add any New generated materials. 
	// The main complication is that we cannot change the slot indices for the existing Source materials,
	// as we would have to fix up the HighRes source. Ideally they are the first N slots, and we
	// just append the new ones. But we cannot guarantee this, so if there are gaps we will interleave
	// the new materials when possible.


	// Figure out which derived materials we are actually going to use.
	// We will not use materials we can re-use from the source, or materials that were previously auto-generated.
	int32 NumMaterials = SourceMaterials.Num();
	TArray<int32> DerivedMatSlotIndexMap;		// this maps from current DerivedMesh slot indices to their final slot indices
	DerivedMatSlotIndexMap.SetNum(NumMaterials);
	TArray<int32> DerivedMaterialsToAdd;		// list of derived material indices we need to store in the final material set
	int32 NewMaterialCount = 0;
	for (int32 mi = 0; mi < NumMaterials; ++mi)
	{
		if (SourceMaterials[mi].bIsPreviouslyGeneratedMaterial)
		{
			DerivedMatSlotIndexMap[mi] = -2;		// these materials do not appear in DerivedMesh or SourceMesh and should be skipped/discarded (todo: and deleted?)
		}
		else  if (SourceMaterials[mi].bIsReusable
				  || SourceMaterials[mi].Constraint == EMaterialBakingConstraint::UseExistingMaterial)	
		{
			DerivedMatSlotIndexMap[mi] = mi;		// if we can re-use existing material we just rewrite to existing material slot index
		}
		else
		{
			DerivedMatSlotIndexMap[mi] = -1;		// will need to allocate a new slot for this material
			DerivedMaterialsToAdd.Add(mi);
			NewMaterialCount++;
		}
	}
	int32 CurRemainingDerivedIdx = 0;

	// Copy existing materials we want to keep to new StaticMesh materials set. 
	// If there are any gaps left by skipping previously-derived materials, try to 
	// tuck in a new derived material that is waiting to be allocated to a slot
	TArray<FStaticMaterial> NewMaterialSet;
	TArray<int32> DerivedMaterialSlotIndices;
	for (int32 k = 0; k < SourceMaterials.Num(); ++k)
	{
		if (SourceMaterials[k].bIsPreviouslyGeneratedMaterial == false)
		{
			NewMaterialSet.Add(SourceMaterials[k].SourceMaterial);
		}
		else if (CurRemainingDerivedIdx < DerivedMaterialsToAdd.Num() )
		{
			int32 DerivedIdx = DerivedMaterialsToAdd[CurRemainingDerivedIdx];
			CurRemainingDerivedIdx++;
			DerivedMatSlotIndexMap[DerivedIdx] = NewMaterialSet.Num();
			DerivedMaterialSlotIndices.Add(DerivedMatSlotIndexMap[DerivedIdx]);
			NewMaterialSet.Add(DerivedMaterials[DerivedIdx].DerivedMaterial);
		}
		else
		{
			// we ran out of new materials to allocate and so just add empty ones??
			ensure(false);
			NewMaterialSet.Add(FStaticMaterial());
		}
	}

	// if we have any new derived materials left, append them to the material set
	while (CurRemainingDerivedIdx < DerivedMaterialsToAdd.Num())
	{
		int32 DerivedIdx = DerivedMaterialsToAdd[CurRemainingDerivedIdx];
		CurRemainingDerivedIdx++;
		DerivedMatSlotIndexMap[DerivedIdx] = NewMaterialSet.Num();
		DerivedMaterialSlotIndices.Add(DerivedMatSlotIndexMap[DerivedIdx]);
		NewMaterialSet.Add(DerivedMaterials[DerivedIdx].DerivedMaterial);
	}

	// apply the material slot index rewrite map to the DerivedMesh
	DerivedLODMesh.Attributes()->EnableMaterialID();
	FDynamicMeshMaterialAttribute* MaterialIDs = DerivedLODMesh.Attributes()->GetMaterialID();
	for (int32 tid : DerivedLODMesh.TriangleIndicesItr())
	{
		int32 CurMaterialID = MaterialIDs->GetValue(tid);
		int32 NewMaterialID = DerivedMatSlotIndexMap[CurMaterialID];
		if (ensure(NewMaterialID >= 0))
		{
			MaterialIDs->SetValue(tid, NewMaterialID);
		}
	}

	// update materials on generated mesh
	SourceStaticMesh->SetStaticMaterials(NewMaterialSet);

	// store new derived LOD as LOD 0
	SourceStaticMesh->SetNumSourceModels(1);
	FMeshDescription* MeshDescription = SourceStaticMesh->GetMeshDescription(0);
	if (MeshDescription == nullptr)
	{
		MeshDescription = SourceStaticMesh->CreateMeshDescription(0);
	}
	FConversionToMeshDescriptionOptions ConversionOptions;
	FDynamicMeshToMeshDescription Converter(ConversionOptions);
	Converter.Convert(&DerivedLODMesh, *MeshDescription);

	// calculate tangents
	Converter.UpdateTangents(&DerivedLODMesh, *MeshDescription, &DerivedLODMeshTangents);

	// set slot names on MeshDescription to match those we set on the generated FStaticMaterials, 
	// because StaticMesh RenderBuffer setup will do matching-name lookups and if it is NAME_None
	// we will get the wrong material!
	FStaticMeshAttributes Attributes(*MeshDescription);
	TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();
	for (int32 SlotIdx : DerivedMaterialSlotIndices)
	{
		// It's possible that NewMaterialSet.Num() > PolygonGroupImportedMaterialSlotNames.GetNumElements()
		// if there are new materials that aren't referenced by any triangles...
		if (SlotIdx < PolygonGroupImportedMaterialSlotNames.GetNumElements())		
		{
			PolygonGroupImportedMaterialSlotNames.Set(SlotIdx, NewMaterialSet[SlotIdx].ImportedMaterialSlotName);
		}
	}

	// Disable auto-generated normals/tangents, we need to use the ones we computed in LOD Generator
	SrcModel.BuildSettings.bRecomputeNormals = false;
	SrcModel.BuildSettings.bRecomputeTangents = false;

	// this will prevent simplification?
	SrcModel.ReductionSettings.MaxDeviation = 0.0f;
	SrcModel.ReductionSettings.PercentTriangles = 1.0f;
	SrcModel.ReductionSettings.PercentVertices = 1.0f;

	// commit update
	SourceStaticMesh->CommitMeshDescription(0);

	// collision
	FPhysicsDataCollection NewCollisionGeo;
	NewCollisionGeo.Geometry = DerivedCollision;
	NewCollisionGeo.CopyGeometryToAggregate();

	// code below derived from FStaticMeshEditor::DuplicateSelectedPrims()
	UBodySetup* BodySetup = SourceStaticMesh->GetBodySetup();
	// mark the BodySetup for modification. Do we need to modify the UStaticMesh??
	BodySetup->Modify();
	//Clear the cache (PIE may have created some data), create new GUID    (comment from StaticMeshEditor)
	BodySetup->InvalidatePhysicsData();
	BodySetup->RemoveSimpleCollision();
	BodySetup->AggGeom = NewCollisionGeo.AggGeom;
	// update collision type
	BodySetup->CollisionTraceFlag = ECollisionTraceFlag::CTF_UseDefault;
	// rebuild physics data
	BodySetup->InvalidatePhysicsData();
	BodySetup->CreatePhysicsMeshes();

	// do we need to do a post edit change here??

	SourceStaticMesh->RecreateNavCollision();

	// save UUID used for generated Assets
	SourceStaticMesh->GetPackage()->GetMetaData().SetValue(SourceStaticMesh, TEXT("StaticMeshLOD.GenerationGUID"), *DerivedAssetGUIDKey);

	GEditor->EndTransaction();

	// done updating mesh
	SourceStaticMesh->PostEditChange();
}



#undef LOCTEXT_NAMESPACE
