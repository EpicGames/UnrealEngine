// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshLODToolsetModule.h"
#include "Misc/CoreDelegates.h"
#include "Tools/LODGenerationSettingsAsset.h"
#include "PropertyEditorModule.h"
#include "Tools/DetailsCustomizations/AutoLODToolCustomizations.h"
#include "Tools/GenerateStaticMeshLODAssetTool.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FMeshLODToolsetModule"

DEFINE_LOG_CATEGORY(LogMeshLODToolset);

// This function will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module
void FMeshLODToolsetModule::StartupModule()
{
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FMeshLODToolsetModule::OnPostEngineInit);
}

// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
// we call this function before unloading the module.
void FMeshLODToolsetModule::ShutdownModule()
{
	// Unregister customizations
	FPropertyEditorModule* PropertyEditorModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");
	if (PropertyEditorModule)
	{
		for (FName ClassName : ClassesToUnregisterOnShutdown)
		{
			PropertyEditorModule->UnregisterCustomClassLayout(ClassName);
		}
	}
}

void FMeshLODToolsetModule::OnPostEngineInit()
{
	ClassesToUnregisterOnShutdown.Reset();

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyModule.RegisterCustomClassLayout("GenerateStaticMeshLODAssetToolProperties", FOnGetDetailCustomizationInstance::CreateStatic(&FAutoLODToolDetails::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UGenerateStaticMeshLODAssetToolProperties::StaticClass()->GetFName());
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FMeshLODToolsetModule, MeshLODToolset)
