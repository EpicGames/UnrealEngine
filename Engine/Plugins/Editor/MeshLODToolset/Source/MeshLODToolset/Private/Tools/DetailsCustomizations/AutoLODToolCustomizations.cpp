// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/DetailsCustomizations/AutoLODToolCustomizations.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailGroup.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Tools/GenerateStaticMeshLODAssetTool.h"

#define LOCTEXT_NAMESPACE "AutoLODToolCustomizations"

TSharedRef<IDetailCustomization> FAutoLODToolDetails::MakeInstance()
{
	return MakeShareable(new FAutoLODToolDetails);
}

void FAutoLODToolDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TSharedRef<IPropertyHandle> PreprocessingGroupSettings = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UGenerateStaticMeshLODAssetToolProperties, Preprocessing), 
		UGenerateStaticMeshLODAssetToolProperties::StaticClass());
	
	// NOTE: We are completely rebuilding the widgets for the FGenerateStaticMeshLODProcess_PreprocessSettings struct. If the struct changes, this code will
	// need to be updated.
	 	
	// Hide existing group
	DetailBuilder.HideProperty(PreprocessingGroupSettings);

	// Create new group
	IDetailCategoryBuilder& ConfigCategory = DetailBuilder.EditCategory("Generator Configuration");

	IDetailGroup& DetailGroup = ConfigCategory.AddGroup("Preprocessing", LOCTEXT("Preprocessing", "Preprocessing"));

	TSharedPtr<IPropertyHandle> FilterGroupLayerHandle = PreprocessingGroupSettings->GetChildHandle(FName("FilterGroupLayer"));
	check(FilterGroupLayerHandle);

	DetailGroup.AddWidgetRow()
	.NameContent()
	[
		FilterGroupLayerHandle->CreatePropertyNameWidget(LOCTEXT("DetailFilterGroupLayerName", "Detail Filter Group Layer"))
	]
	.ValueContent()
	.VAlign(VAlign_Center)
	[
		SNew(SEditableTextBox)
		.Text(FText())
		.HintText(LOCTEXT("DetailFilterGroupLayerValueHintText", "(e.g. PreFilterGroups)"))
		.OnTextCommitted_Lambda([FilterGroupLayerHandle](const FText& Val, ETextCommit::Type TextCommitType)
		{
			FilterGroupLayerHandle->SetValue(Val.ToString());
		})
	];

	TSharedPtr<IPropertyHandle> ThickenWeightMapNameHandle = PreprocessingGroupSettings->GetChildHandle(FName("ThickenWeightMapName"));
	check(ThickenWeightMapNameHandle);

	DetailGroup.AddWidgetRow()
	.NameContent()
	[
		ThickenWeightMapNameHandle->CreatePropertyNameWidget(LOCTEXT("ThickenWeightMapName", "Thicken Weight Map"))
	]
	.ValueContent()
	.VAlign(VAlign_Center)
	[
		SNew(SEditableTextBox)
		.Text(FText())
		.HintText(LOCTEXT("ThickenWeightMapNameValueHintText", "(e.g. ThickenMap)"))
		.OnTextCommitted_Lambda([ThickenWeightMapNameHandle](const FText& Val, ETextCommit::Type TextCommitType)
		{
			ThickenWeightMapNameHandle->SetValue(Val.ToString());
		})
	];

	TSharedPtr<IPropertyHandle> ThickenAmountHandle = PreprocessingGroupSettings->GetChildHandle(FName("ThickenAmount"));
	check(ThickenAmountHandle);

	DetailGroup.AddPropertyRow(ThickenAmountHandle.ToSharedRef());
}

#undef LOCTEXT_NAMESPACE

