// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/GenerateStaticMeshLODAssetTool.h"
#include "Drawing/LineSetComponent.h"
#include "InteractiveToolManager.h"

#include "Drawing/PreviewGeometryActor.h"
#include "ToolSetupUtil.h"


#include "Materials/MaterialInterface.h"
#include "Util/ColorConstants.h"



#include "Physics/CollisionPropertySets.h"
#include "Physics/PhysicsDataCollection.h"
#include "Physics/CollisionGeometryVisualization.h"

#include "Components/StaticMeshComponent.h"

#include "Misc/Paths.h"
#include "EditorAssetLibrary.h"
#include "ObjectTools.h"

#include "ModelingOperators.h"
#include "MeshOpPreviewHelpers.h"
#include "Generators/GridBoxMeshGenerator.h"

#include "Framework/Notifications/NotificationManager.h"
#include "Framework/Docking/TabManager.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "TargetInterfaces/StaticMeshBackedTarget.h"
#include "ToolContextInterfaces.h"
#include "ToolTargetManager.h"
#include "ModelingToolTargetUtil.h"

#include "Tools/LODGenerationSettingsAsset.h"

static_assert(WITH_EDITOR, "Tool being compiled without editor");
#include "Misc/ScopedSlowTask.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GenerateStaticMeshLODAssetTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UGenerateStaticMeshLODAssetTool"

//
// Local Op Stuff
//

namespace GenerateStaticMeshLODAssetLocals
{

	class FGenerateStaticMeshLODAssetOperatorOp : public FDynamicMeshOperator, public FGCObject
	{
	public:

		// Inputs
		TObjectPtr<UGenerateStaticMeshLODProcess> GenerateProcess;
		FGenerateStaticMeshLODProcess_PreprocessSettings GeneratorSettings_Preprocess;
		FGenerateStaticMeshLODProcessSettings GeneratorSettings_MeshGeneration;
		FGenerateStaticMeshLODProcess_SimplifySettings GeneratorSettings_Simplify;
		FGenerateStaticMeshLODProcess_NormalsSettings GeneratorSettings_Normals;
		FGenerateStaticMeshLODProcess_TextureSettings GeneratorSettings_Texture;
		FGenerateStaticMeshLODProcess_UVSettings GeneratorSettings_UV;
		FGenerateStaticMeshLODProcess_CollisionSettings GeneratorSettings_Collision;
		TArray<TPair<UMaterialInterface*, UGenerateStaticMeshLODProcess::EMaterialBakingConstraint>> MaterialConstraints;
		TArray<TPair<UTexture2D*, UGenerateStaticMeshLODProcess::ETextureBakingConstraint>> TextureConstraints;
		
		// Outputs
		// Inherited: 	TUniquePtr<FDynamicMesh3> ResultMesh;
		// 				FTransform3d ResultTransform;
		FMeshTangentsd ResultTangents;
		FSimpleShapeSet3d ResultCollision;

		void AddReferencedObjects(FReferenceCollector& Collector) override
		{
			Collector.AddReferencedObject(GenerateProcess);
		}
		virtual FString GetReferencerName() const override
		{
			return TEXT("GenerateStaticMeshLODAssetLocals::FGenerateStaticMeshLODAssetOperatorOp");
		}

		void CalculateResult(FProgressCancel* Progress) override
		{
			auto DoCompute = [this](FProgressCancel* Progress)		// bracket this computation with lock/unlock
			{
				if (Progress && Progress->Cancelled())
				{
					return;
				}

				GenerateProcess->UpdatePreprocessSettings(GeneratorSettings_Preprocess);
				GenerateProcess->UpdateSettings(GeneratorSettings_MeshGeneration);
				GenerateProcess->UpdateSimplifySettings(GeneratorSettings_Simplify);
				GenerateProcess->UpdateNormalsSettings(GeneratorSettings_Normals);
				GenerateProcess->UpdateTextureSettings(GeneratorSettings_Texture);
				GenerateProcess->UpdateUVSettings(GeneratorSettings_UV);
				GenerateProcess->UpdateCollisionSettings(GeneratorSettings_Collision);
				for (TPair<UMaterialInterface*, UGenerateStaticMeshLODProcess::EMaterialBakingConstraint> MatConstraint : MaterialConstraints)
				{
					GenerateProcess->UpdateSourceBakeMaterialConstraint(MatConstraint.Key, MatConstraint.Value);
				}
				for (TPair<UTexture2D*, UGenerateStaticMeshLODProcess::ETextureBakingConstraint> TexConstraint : TextureConstraints)
				{
					GenerateProcess->UpdateSourceBakeTextureConstraint(TexConstraint.Key, TexConstraint.Value);
				}

				if (Progress && Progress->Cancelled())
				{
					return;
				}

				GenerateProcess->ComputeDerivedSourceData(Progress);

				if (Progress && Progress->Cancelled())
				{
					return;
				}

				*ResultMesh = GenerateProcess->GetDerivedLOD0Mesh();
				ResultTangents = GenerateProcess->GetDerivedLOD0MeshTangents();
				ResultCollision = GenerateProcess->GetDerivedCollision();

				if (ResultMesh->HasAttributes() && ResultTangents.GetTangents().Num() > 0 && ResultTangents.GetBitangents().Num() > 0)
				{
					ResultMesh->Attributes()->SetNumNormalLayers(3);
					ensure(ResultTangents.CopyToOverlays(*ResultMesh));
				}
			};

			GenerateProcess->GraphEvalCriticalSection.Lock();
			DoCompute(Progress);
			GenerateProcess->GraphEvalCriticalSection.Unlock();
		}
	};

	class FGenerateStaticMeshLODAssetOperatorFactory : public IDynamicMeshOperatorFactory
	{

	public:

		FGenerateStaticMeshLODAssetOperatorFactory(UGenerateStaticMeshLODAssetTool* AutoLODTool, FTransformSRT3d ResultTransform) :
			AutoLODTool(AutoLODTool), 
			ResultTransform(ResultTransform) 
		{}

		// IDynamicMeshOperatorFactory API
		virtual TUniquePtr<FDynamicMeshOperator> MakeNewOperator() override
		{
			check(AutoLODTool);
			TUniquePtr<FGenerateStaticMeshLODAssetOperatorOp> Op = MakeUnique<FGenerateStaticMeshLODAssetOperatorOp>();		
			Op->GenerateProcess = AutoLODTool->GenerateProcess;
			Op->GeneratorSettings_Preprocess = AutoLODTool->BasicProperties->Preprocessing;
			Op->GeneratorSettings_MeshGeneration = AutoLODTool->BasicProperties->MeshGeneration;
			Op->GeneratorSettings_Simplify = AutoLODTool->BasicProperties->Simplification;
			Op->GeneratorSettings_Normals = AutoLODTool->BasicProperties->Normals;
			Op->GeneratorSettings_Texture = AutoLODTool->BasicProperties->TextureBaking;
			Op->GeneratorSettings_UV = AutoLODTool->BasicProperties->UVGeneration;
			Op->GeneratorSettings_Collision.CollisionGroupLayerName = AutoLODTool->BasicProperties->CollisionGroupLayerName;
			Op->GeneratorSettings_Collision = AutoLODTool->BasicProperties->SimpleCollision;

			Op->MaterialConstraints.Reset();
			for (const FGenerateStaticMeshLOD_MaterialConfig& MatConfig : AutoLODTool->TextureProperties->Materials)
			{
				Op->MaterialConstraints.Add( TPair<UMaterialInterface*, UGenerateStaticMeshLODProcess::EMaterialBakingConstraint>( 
					MatConfig.Material, static_cast<UGenerateStaticMeshLODProcess::EMaterialBakingConstraint>(MatConfig.Constraint) ) );
			}
			Op->TextureConstraints.Reset();
			for (const FGenerateStaticMeshLOD_TextureConfig& TexConfig : AutoLODTool->TextureProperties->Textures)
			{
				Op->TextureConstraints.Add( TPair<UTexture2D*, UGenerateStaticMeshLODProcess::ETextureBakingConstraint>( 
					TexConfig.Texture, static_cast<UGenerateStaticMeshLODProcess::ETextureBakingConstraint>(TexConfig.Constraint) ) );
			}

			Op->SetResultTransform(ResultTransform);
			return Op;
		}

		UGenerateStaticMeshLODAssetTool* AutoLODTool = nullptr;
		FTransformSRT3d ResultTransform;
	};

	static void DisplayCriticalWarningMessage(const FString& Message)
	{
		FNotificationInfo Info(FText::FromString(Message));
		Info.ExpireDuration = 5.0f;
		FSlateNotificationManager::Get().AddNotification(Info);

		UE_LOGF(LogTemp, Warning, "%ls", *Message);
	}
}


/*
 * ToolBuilder
 */


const FToolTargetTypeRequirements& UGenerateStaticMeshLODAssetToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UPrimitiveComponentBackedTarget::StaticClass(),
		UStaticMeshBackedTarget::StaticClass()
		});
	return TypeRequirements;
}

bool UGenerateStaticMeshLODAssetToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	// hack to make multi-tool look like single-tool
	const int NumTargets = SceneState.TargetManager->CountSelectedAndTargetable(SceneState, GetTargetRequirements());
	if (NumTargets == 1)
	{
		// Ensure that the StaticMesh asset is still valid.
		// [TODO] Remove this check once we properly handle a failed DuplicateAsset in UGenerateStaticMeshLODProcess::WriteDerivedStaticMeshAsset
		bool bValidTarget = false;
		SceneState.TargetManager->EnumerateSelectedAndTargetableComponents(SceneState, GetTargetRequirements(),
			[&bValidTarget](UActorComponent* Component)
			{
				if (const UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component))
				{
					UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
					if (IsValid(StaticMesh))
					{
						bValidTarget = ObjectTools::IsObjectBrowsable(StaticMesh);
					}
				}
			});
		return bValidTarget;
	}
	return false;
}

UMultiSelectionMeshEditingTool* UGenerateStaticMeshLODAssetToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	UGenerateStaticMeshLODAssetTool* NewTool = NewObject<UGenerateStaticMeshLODAssetTool>(SceneState.ToolManager);
	NewTool->SetUseAssetEditorMode(bUseAssetEditorMode);
	NewTool->SetRestrictiveMode(bInRestrictiveMode);
	return NewTool;
}




void UGenerateStaticMeshLODAssetToolPresetProperties::PostAction(EGenerateLODAssetToolPresetAction Action)
{
	if (ParentTool.IsValid())
	{
		ParentTool->RequestPresetAction(Action);
	}
}


/*
 * Tool
 */


void UGenerateStaticMeshLODAssetTool::SetUseAssetEditorMode(bool bEnable)
{
	bIsInAssetEditorMode = bEnable;
}


void UGenerateStaticMeshLODAssetTool::SetRestrictiveMode(bool bEnable)
{
	bRestrictiveMode = bEnable;
}


void UGenerateStaticMeshLODAssetTool::Setup()
{
	using GenerateStaticMeshLODAssetLocals::FGenerateStaticMeshLODAssetOperatorFactory;

	UInteractiveTool::Setup();

	SetToolDisplayName(LOCTEXT("ToolName", "AutoLOD"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartStaticMeshLODAssetTool", "Create a new LOD asset"),
		EToolMessageLevel::UserNotification);

	GenerateProcess = NewObject<UGenerateStaticMeshLODProcess>(this);

	UStaticMeshComponent* StaticMeshComponent = CastChecked<UStaticMeshComponent>(UE::ToolTarget::GetTargetComponent(Targets[0]));
	UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();

	FComponentMaterialSet ComponentMaterials = UE::ToolTarget::GetMaterialSet(Targets[0], false);
	FComponentMaterialSet AssetMaterials = UE::ToolTarget::GetMaterialSet(Targets[0], true);
	if (ComponentMaterials != AssetMaterials)
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("GenerateStaticMeshLODAssetTool_DifferentMaterials", "Selected Component and StaticMesh Asset have different Material Sets. Asset Materials will be used."),
			EToolMessageLevel::UserWarning);
	}
	

	FProgressCancel Progress;
	FScopedSlowTask SlowTask(2, LOCTEXT("UGenerateStaticMeshLODAssetTool_Setup", "Initializing AutoLOD Generator ..."));
	SlowTask.MakeDialog();

	if (StaticMesh)
	{
		SlowTask.EnterProgressFrame(1);

		bool bInitializeOK = GenerateProcess->Initialize(StaticMesh, &Progress);		// Must happen on main thread

		if (Progress.Warnings.Num() > 0)
		{
			const FProgressCancel::FMessageInfo& Warning = Progress.Warnings[0];
			GetToolManager()->DisplayMessage(Warning.MessageText, (EToolMessageLevel)Warning.MessageLevel);
		}

		if (!bInitializeOK)
		{
			GenerateProcess = nullptr;
			GetToolManager()->DisplayMessage(
				LOCTEXT("GenerateStaticMeshLODAssetTool_ErrorInitializing", "Error initializing tool process: invalid Static Mesh input"),
				EToolMessageLevel::UserError);
			return;
		}

	}
	else
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("GenerateStaticMeshLODAssetTool_NoStaticMesh", "Could not find Static Mesh in selected input"),
			EToolMessageLevel::UserError);
		return;
	}

	FString FullPathWithExtension = UEditorAssetLibrary::GetPathNameForLoadedAsset(StaticMesh);
	OutputProperties = NewObject<UGenerateStaticMeshLODAssetToolOutputProperties>(this);
	AddToolPropertySource(OutputProperties);
	OutputProperties->NewAssetName = FPaths::GetBaseFilename(FullPathWithExtension, true);
	OutputProperties->GeneratedSuffix = TEXT("_AutoLOD");
	OutputProperties->RestoreProperties(this);
	OutputProperties->OutputMode = (bIsInAssetEditorMode) ? EGenerateLODAssetOutputMode::UpdateExistingAsset : EGenerateLODAssetOutputMode::CreateNewAsset;
	OutputProperties->bShowOutputMode = !bIsInAssetEditorMode;

	if (!bRestrictiveMode)
	{
		PresetProperties = NewObject<UGenerateStaticMeshLODAssetToolPresetProperties>(this);
		PresetProperties->RestoreProperties(this);
		PresetProperties->Initialize(this);
		AddToolPropertySource(PresetProperties);
		PresetProperties->WatchProperty(PresetProperties->Preset, [this](TWeakObjectPtr<UStaticMeshLODGenerationSettings>) { OnPresetSelectionChanged(); });
	}

	SlowTask.EnterProgressFrame(1);
	BasicProperties = NewObject<UGenerateStaticMeshLODAssetToolProperties>(this);
	AddToolPropertySource(BasicProperties);
	BasicProperties->Preprocessing = GenerateProcess->GetCurrentPreprocessSettings();
	BasicProperties->MeshGeneration = GenerateProcess->GetCurrentSettings();
	BasicProperties->Simplification = GenerateProcess->GetCurrentSimplifySettings();
	BasicProperties->Normals = GenerateProcess->GetCurrentNormalsSettings();
	BasicProperties->TextureBaking = GenerateProcess->GetCurrentTextureSettings();
	BasicProperties->UVGeneration = GenerateProcess->GetCurrentUVSettings();
	BasicProperties->SimpleCollision = GenerateProcess->GetCurrentCollisionSettings();
	// if we defer restore to here, then on first run we get the defaults, but afterwards we get the restored values
	BasicProperties->RestoreProperties(this);

	// assume this is an Editor-only Tool, so we can rely on this
	BasicProperties->GetOnModified().AddLambda([this](UObject*, FProperty*) { OnSettingsModified(); });

	// Collision layer name property
	BasicProperties->WatchProperty(BasicProperties->CollisionGroupLayerName, [this](FName) { OnSettingsModified(); });
	BasicProperties->InitializeGroupLayers(&(GenerateProcess->GetSourceMesh()));

	TextureProperties = NewObject<UGenerateStaticMeshLODAssetToolTextureProperties>(this);
	AddToolPropertySource(TextureProperties);
	for (UTexture2D* BakeTexture : GenerateProcess->GetSourceBakeTextures())
	{
		TextureProperties->Textures.Add( FGenerateStaticMeshLOD_TextureConfig{ BakeTexture, EGenerateStaticMeshLOD_BakeConstraint::NoConstraint } );
	}
	TextureProperties->WatchProperty(TextureProperties->Textures, [this](TArray<FGenerateStaticMeshLOD_TextureConfig> NewValues) { OnSettingsModified(); });
	for (UMaterialInterface* BakeMaterial : GenerateProcess->GetSourceBakeMaterials())
	{
		TextureProperties->Materials.Add( FGenerateStaticMeshLOD_MaterialConfig{ BakeMaterial, EGenerateStaticMeshLOD_BakeConstraint::NoConstraint } );
	}
	TextureProperties->WatchProperty(TextureProperties->Materials, [this](TArray<FGenerateStaticMeshLOD_MaterialConfig> NewValues) { OnSettingsModified(); });


	FBoxSphereBounds Bounds = StaticMeshComponent->Bounds;
	FTransformSRT3d PreviewTransform = UE::ToolTarget::GetLocalToWorldTransform(Targets[0]);
	PreviewTransform.SetTranslation(PreviewTransform.GetTranslation() + 2.5 * (double)Bounds.BoxExtent.Y * FVector3d::UnitY());

	this->OpFactory = MakeUnique<FGenerateStaticMeshLODAssetOperatorFactory>(this, PreviewTransform);
	PreviewWithBackgroundCompute = NewObject<UMeshOpPreviewWithBackgroundCompute>(this, "Preview");
	PreviewWithBackgroundCompute->Setup(GetTargetWorld(), this->OpFactory.Get());
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(PreviewWithBackgroundCompute->PreviewMesh, nullptr);
	PreviewWithBackgroundCompute->PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::ExternallyProvided);

	PreviewWithBackgroundCompute->OnMeshUpdated.AddLambda([this](UMeshOpPreviewWithBackgroundCompute* Compute) 
	{
		if (Compute->HaveEmptyResult())
		{
			GetToolManager()->DisplayMessage(LOCTEXT("CannotCreateEmptyMesh", "WARNING: Tool doesn't allow creation of an empty mesh."),
				EToolMessageLevel::UserWarning);
		}
		// Don't clear the message area if we have a non-empty result
	});

	// For the first computation, display a bounding box with the working material. Otherwise it looks like nothing
	// is happening. And we don't want to copy over the potentially huge input mesh to be the preview mesh.
	FGridBoxMeshGenerator MeshGen;
	MeshGen.Box = UE::Geometry::FOrientedBox3d((FVector3d)Bounds.Origin, (FVector3d)Bounds.BoxExtent);
	MeshGen.Generate();
	FDynamicMesh3 BoxMesh(&MeshGen);
	PreviewWithBackgroundCompute->PreviewMesh->UpdatePreview(MoveTemp(BoxMesh));
	PreviewWithBackgroundCompute->PreviewMesh->SetTransform(FTransform(FVector(0, 2.5f * Bounds.BoxExtent.Y, 0)));

	PreviewWithBackgroundCompute->OnOpCompleted.AddLambda([this](const FDynamicMeshOperator* Op)
	{
		const GenerateStaticMeshLODAssetLocals::FGenerateStaticMeshLODAssetOperatorOp* GenerateLODOp =
			(const GenerateStaticMeshLODAssetLocals::FGenerateStaticMeshLODAssetOperatorOp*)(Op);
		check(GenerateLODOp);

		// Clear any previous collision geometry visualization
		CollisionPreview->RemoveAllLineSets();

		// Must happen on main thread
		FPhysicsDataCollection PhysicsData;
		PhysicsData.Geometry = GenerateLODOp->ResultCollision;
		PhysicsData.CopyGeometryToAggregate();
		UE::PhysicsTools::InitializeCollisionGeometryVisualization(CollisionPreview, CollisionVizSettings, PhysicsData, 0.f, 16);

		// Must happen on main thread, and GenerateProcess might be in use by an Op somewhere else
		GenerateProcess->GraphEvalCriticalSection.Lock();

		UGenerateStaticMeshLODProcess::FPreviewMaterials PreviewMaterialSet;
		GenerateProcess->GetDerivedMaterialsPreview(PreviewMaterialSet);
		if (PreviewMaterialSet.Materials.Num() > 0)
		{
			PreviewTextures = PreviewMaterialSet.Textures;
			PreviewMaterials = PreviewMaterialSet.Materials;
			PreviewWithBackgroundCompute->PreviewMesh->SetMaterials(PreviewMaterials);
			TextureProperties->PreviewTextures = PreviewTextures;
		}

		GenerateProcess->GraphEvalCriticalSection.Unlock();
	});

	PreviewWithBackgroundCompute->ConfigureMaterials(
		ToolSetupUtil::GetDefaultSculptMaterial(GetToolManager()),
		ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager())
	);

	CollisionVizSettings = NewObject<UCollisionGeometryVisualizationProperties>(this);
	CollisionVizSettings->RestoreProperties(this);
	AddToolPropertySource(CollisionVizSettings);
	CollisionVizSettings->Initialize(this);

	CollisionPreview = NewObject<UPreviewGeometry>(this);
	CollisionPreview->CreateInWorld(GetTargetWorld(), (FTransform)PreviewTransform);

	// Trigger any automatic Preset-changed behavior if we started Tool with one already selected
	// Note: Currently this does nothing, as we rely on the user to manually read/write the preset
	OnPresetSelectionChanged();

	// Pop up notifications for any warnings
	for ( const FProgressCancel::FMessageInfo& Warning : Progress.Warnings )
	{
		FNotificationInfo NotificationInfo(Warning.MessageText);
		NotificationInfo.ExpireDuration = 6.0f;
		NotificationInfo.Hyperlink = FSimpleDelegate::CreateLambda([]() { FGlobalTabmanager::Get()->TryInvokeTab(FName("OutputLog")); });
		NotificationInfo.HyperlinkText = LOCTEXT("ShowOutputLogHyperlink", "Show Output Log");
		FSlateNotificationManager::Get().AddNotification(NotificationInfo)->SetCompletionState(SNotificationItem::CS_Fail);
	}
}



bool UGenerateStaticMeshLODAssetTool::ValidateSettings() const
{
	if (!BasicProperties || !GenerateProcess)
	{
		return true;
	}

	const FDynamicMesh3& Mesh = GenerateProcess->GetSourceMesh();

	const FName& GroupName = BasicProperties->Preprocessing.FilterGroupLayer;
	if (!GroupName.IsNone())
	{
		bool bFound = false;

		if (Mesh.HasAttributes())
		{
			for (int GroupLayerIndex = 0; GroupLayerIndex < Mesh.Attributes()->NumPolygroupLayers(); ++GroupLayerIndex)
			{
				const FDynamicMeshPolygroupAttribute* GroupAttribute = Mesh.Attributes()->GetPolygroupLayer(GroupLayerIndex);
				if (GroupAttribute->GetName() == GroupName)
				{
					bFound = true;
					break;
				}
			}
		}

		if (!bFound)
		{
			const FText Message = FText::Format(LOCTEXT("GroupNotFoundWarning", "Group {0} not found on input mesh"), FText::FromName(GroupName));
			GetToolManager()->DisplayMessage(Message, EToolMessageLevel::UserWarning);
			return false;
		}
	}


	const FName& WeightMapName = BasicProperties->Preprocessing.ThickenWeightMapName;
	if (!WeightMapName.IsNone())
	{
		bool bFound = false;

		if (Mesh.HasAttributes())
		{
			for (int WeightMapIndex = 0; WeightMapIndex < Mesh.Attributes()->NumWeightLayers(); ++WeightMapIndex)
			{
				const FDynamicMeshWeightAttribute* GroupAttribute = Mesh.Attributes()->GetWeightLayer(WeightMapIndex);
				if (GroupAttribute->GetName() == WeightMapName)
				{
					bFound = true;
					break;
				}
			}
		}

		if (!bFound)
		{
			const FText Message = FText::Format(LOCTEXT("WeightMapNotFoundWarning", "Weight Map {0} not found on input mesh"), FText::FromName(WeightMapName));
			GetToolManager()->DisplayMessage(Message, EToolMessageLevel::UserWarning);
			return false;
		}
	}

	return true;
}

void UGenerateStaticMeshLODAssetTool::OnSettingsModified()
{
	if (BasicProperties)
	{
		// Set the transient property which controls the edit condition for ThickenAmount
		// Note: The tool will display an error if ThickenWeightMapName exists but isn't valid
		BasicProperties->Preprocessing.bIsThickenAmountEnabled = (
			BasicProperties->Preprocessing.ThickenWeightMapName.IsNone() == false);
	}

	bool bOK = ValidateSettings();
	if (bOK)
	{
		GetToolManager()->DisplayMessage({}, EToolMessageLevel::UserWarning);
	}

	PreviewWithBackgroundCompute->InvalidateResult();
}



void UGenerateStaticMeshLODAssetTool::OnShutdown(EToolShutdownType ShutdownType)
{
	if (OutputProperties)
	{
		OutputProperties->SaveProperties(this);
	}
	if (BasicProperties)
	{
		BasicProperties->SaveProperties(this);
	}
	if (CollisionVizSettings)
	{
		CollisionVizSettings->SaveProperties(this);
	}
	if (PresetProperties)
	{
		PresetProperties->SaveProperties(this);
	}
	if (CollisionPreview)
	{
		CollisionPreview->Disconnect();
		CollisionPreview = nullptr;
	}

	if (ShutdownType == EToolShutdownType::Accept && GenerateProcess)
	{
		if (OutputProperties->OutputMode == EGenerateLODAssetOutputMode::UpdateExistingAsset)
		{
			UpdateExistingAsset();
		}
		else
		{
			CreateNewAsset();
		}
	}

	if (PreviewWithBackgroundCompute)
	{
		PreviewWithBackgroundCompute->Shutdown();
		PreviewWithBackgroundCompute = nullptr;
	}

	if (GenerateProcess)
	{
		GenerateProcess = nullptr;
	}
}


bool UGenerateStaticMeshLODAssetTool::CanAccept() const
{
	return (PreviewWithBackgroundCompute && PreviewWithBackgroundCompute->HaveValidNonEmptyResult());
}


void UGenerateStaticMeshLODAssetTool::OnTick(float DeltaTime)
{
	if (PreviewWithBackgroundCompute)
	{
		PreviewWithBackgroundCompute->Tick(DeltaTime);
	}

	UE::PhysicsTools::UpdateCollisionGeometryVisualization(CollisionPreview, CollisionVizSettings);
}



void UGenerateStaticMeshLODAssetTool::CreateNewAsset()
{
	check(PreviewWithBackgroundCompute->HaveValidResult());
	GenerateProcess->UpdateDerivedPathName(OutputProperties->NewAssetName, OutputProperties->GeneratedSuffix);

	check(GenerateProcess->GraphEvalCriticalSection.TryLock());		// No ops should be running
	GenerateProcess->WriteDerivedAssetData();
	GenerateProcess->GraphEvalCriticalSection.Unlock();
}



void UGenerateStaticMeshLODAssetTool::UpdateExistingAsset()
{
	check(PreviewWithBackgroundCompute->HaveValidResult());

	const UStaticMesh* SourceStaticMesh = GenerateProcess->GetSourceStaticMesh();
	if (SourceStaticMesh->GetPathName().StartsWith(TEXT("/Engine/")))
	{
		GenerateStaticMeshLODAssetLocals::DisplayCriticalWarningMessage(FString::Printf(TEXT("CANNOT MODIFY BUILT-IN ENGINE ASSET %s"), *SourceStaticMesh->GetPathName()));
		return;
	}


	GenerateProcess->UpdateDerivedPathName(OutputProperties->NewAssetName, OutputProperties->GeneratedSuffix);

	check(GenerateProcess->GraphEvalCriticalSection.TryLock());		// No ops should be running

	// only updated HD source if we have no HD source asset. Otherwise we are overwriting with existing lowpoly LOD0.
	bool bUpdateHDSource =
		OutputProperties->bSaveInputAsHiResSource &&
		(GenerateProcess->GetSourceStaticMesh()->IsHiResMeshDescriptionValid() == false);

	GenerateProcess->UpdateSourceAsset(bUpdateHDSource);
	GenerateProcess->GraphEvalCriticalSection.Unlock();
}




void UGenerateStaticMeshLODAssetTool::RequestPresetAction(EGenerateLODAssetToolPresetAction ActionType)
{
	if (PresetProperties == nullptr)
	{
		return;
	}

	UStaticMeshLODGenerationSettings* CurrentPreset = PresetProperties->Preset.Get();
	if (CurrentPreset == nullptr)
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("GenerateStaticMeshLODAssetTool_NoPresetSelected", "No Preset Asset is currently set in the Preset Settings"), EToolMessageLevel::UserError);
		return;
	}

	if (ActionType == EGenerateLODAssetToolPresetAction::ReadFromPreset)
	{
		UStaticMeshLODGenerationSettings* ApplyPreset = PresetProperties->Preset.Get();
		if (ApplyPreset)
		{
			BasicProperties->Preprocessing = ApplyPreset->Preprocessing;
			BasicProperties->MeshGeneration = ApplyPreset->MeshGeneration;
			BasicProperties->Simplification = ApplyPreset->Simplification;
			BasicProperties->Normals = ApplyPreset->Normals;
			BasicProperties->TextureBaking = ApplyPreset->TextureBaking;
			BasicProperties->UVGeneration = ApplyPreset->UVGeneration;
			BasicProperties->SimpleCollision = ApplyPreset->SimpleCollision;
			OnSettingsModified();
		}
	}
	else if (ActionType == EGenerateLODAssetToolPresetAction::WriteToPreset)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("WriteToPresetAction", "Write Preset"));

		CurrentPreset->SetFlags(RF_Transactional);
		CurrentPreset->Modify();

		CurrentPreset->Preprocessing = BasicProperties->Preprocessing;
		CurrentPreset->MeshGeneration = BasicProperties->MeshGeneration;
		CurrentPreset->Simplification = BasicProperties->Simplification;
		CurrentPreset->Normals = BasicProperties->Normals;
		CurrentPreset->TextureBaking = BasicProperties->TextureBaking;
		CurrentPreset->UVGeneration = BasicProperties->UVGeneration;
		CurrentPreset->SimpleCollision = BasicProperties->SimpleCollision;

		CurrentPreset->PostEditChange();

		GetToolManager()->EndUndoTransaction();
	}
}


void UGenerateStaticMeshLODAssetTool::OnPresetSelectionChanged()
{
	// Rely on user to decide when to write/read settings to/from the selected preset
}





#undef LOCTEXT_NAMESPACE
