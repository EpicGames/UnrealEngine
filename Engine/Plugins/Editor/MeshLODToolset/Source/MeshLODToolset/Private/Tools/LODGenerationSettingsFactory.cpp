// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/LODGenerationSettingsFactory.h"
#include "Tools/LODGenerationSettingsAsset.h"
#include "AssetTypeCategories.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LODGenerationSettingsFactory)

#define LOCTEXT_NAMESPACE "UStaticMeshLODGenerationSettingsFactory"

UStaticMeshLODGenerationSettingsFactory::UStaticMeshLODGenerationSettingsFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UStaticMeshLODGenerationSettings::StaticClass();
	bCreateNew = true;
	bText = true;
	bEditAfterNew = true;
	bEditorImport = false;
}

UObject* UStaticMeshLODGenerationSettingsFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UStaticMeshLODGenerationSettings* NewSettings = NewObject<UStaticMeshLODGenerationSettings>(InParent, Class, Name, Flags);
	check(NewSettings);
	return NewSettings;
}


uint32 UStaticMeshLODGenerationSettingsFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::None;
}

FText UStaticMeshLODGenerationSettingsFactory::GetDisplayName() const
{
	return LOCTEXT("MenuEntry", "AutoLOD Settings");
}

FString UStaticMeshLODGenerationSettingsFactory::GetDefaultNewAssetName() const
{
	return TEXT("AutoLODSettings");
}

#undef LOCTEXT_NAMESPACE
