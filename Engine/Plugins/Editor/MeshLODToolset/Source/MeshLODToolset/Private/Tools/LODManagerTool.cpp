// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/LODManagerTool.h"
#include "Drawing/LineSetComponent.h"
#include "InteractiveToolManager.h"

#include "AssetUtils/MeshDescriptionUtil.h"
#include "Drawing/PreviewGeometryActor.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "Engine/StaticMeshSourceData.h"
#include "IndexedMeshToDynamicMesh.h"
#include "ModelingToolTargetUtil.h"

#include "PreviewMesh.h"
#include "StaticMeshAttributes.h"

// for lightmap access
#include "Components/StaticMeshComponent.h"


#include "StaticMeshResources.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "TargetInterfaces/StaticMeshBackedTarget.h"
#include "ToolContextInterfaces.h"
#include "ToolTargetManager.h"
#include "ToolTargets/StaticMeshComponentToolTarget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LODManagerTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "ULODManagerTool"

/*
 * ToolBuilder
 */


const FToolTargetTypeRequirements& ULODManagerToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UMaterialProvider::StaticClass(),
		UPrimitiveComponentBackedTarget::StaticClass(),
		UStaticMeshBackedTarget::StaticClass()
		});
	return TypeRequirements;
}

bool ULODManagerToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	// disable multi-selection for now
	return SceneState.TargetManager->CountSelectedAndTargetable(SceneState, GetTargetRequirements()) == 1;
}

UMultiSelectionMeshEditingTool* ULODManagerToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<ULODManagerTool>(SceneState.ToolManager);
}




void ULODManagerActionPropertySet::PostAction(ELODManagerToolActions Action)
{
	if (ParentTool.IsValid() && Cast<ULODManagerTool>(ParentTool))
	{
		Cast<ULODManagerTool>(ParentTool)->RequestAction(Action);
	}
}

namespace UE {
namespace Geometry {
namespace LODManagerHelper
{

	// Static functions used to query a UStaticMesh and generate FDynamicMesh3 representations of the LODs 
	struct FMeshLODUtils
	{
		// return true if the static mesh has an imported mesh at the specified LODIndex
		static bool HasImportedLOD(const UStaticMesh& StaticMesh, int32 LODIndex)
		{
			const FMeshDescription* LODMesh = StaticMesh.GetMeshDescription(LODIndex);
			return (LODMesh != nullptr);
		}
		
		// get the specified imported LOD.  the result will be a invalid SharedPtr if the given static mesh does not have
		// an imported LOD at the specified index.
		static TSharedPtr<FDynamicMesh3> GetImportedLOD(const UStaticMesh& StaticMesh, int32 LODIndex)
		{
			TSharedPtr<FDynamicMesh3> Result;
			if (FMeshDescription* LODMeshDescription = StaticMesh.GetMeshDescription(LODIndex)) 
			{
				Result = MakeShared<FDynamicMesh3>();
				// TODO: we only need to do this copy if normals or tangents are set to auto-generated...
				FMeshBuildSettings LODBuildSettings = StaticMesh.GetSourceModel(LODIndex).BuildSettings;
				FMeshDescription TmpMeshDescription(*LODMeshDescription);
				UE::MeshDescription::InitializeAutoGeneratedAttributes(TmpMeshDescription, &LODBuildSettings);

				FMeshDescriptionToDynamicMesh Converter;
				Converter.Convert(&TmpMeshDescription, *Result, true);
			}
			return Result;
		}

		// return true if the StaticMesh has render data for the indicated LODIndex 
		static bool HasRenderLOD(const UStaticMesh& StaticMesh, int32 LODIndex)
		{
			if (!StaticMesh.HasValidRenderData())
			{
				return false;
			}
			const FStaticMeshRenderData* RenderData = StaticMesh.GetRenderData();
			return RenderData->LODResources.IsValidIndex(LODIndex);
		}

		// get a dynamic mesh computed from the render data at the specified LODIndex. will return invalid SharedPtr if the
		// LOD doesn't exist.
		static TSharedPtr<FDynamicMesh3> GetRenderLOD(const UStaticMesh& StaticMesh, int32 LODIndex)
		{
			TSharedPtr<FDynamicMesh3> Result;
			if (!FMeshLODUtils::HasRenderLOD(StaticMesh, LODIndex))
			{
				return Result;
			}

			Result = MakeShared<FDynamicMesh3>();
			const FStaticMeshRenderData* RenderData = StaticMesh.GetRenderData();
			const FStaticMeshLODResources& LODResource = RenderData->LODResources[LODIndex];
			UE::Conversion::RenderBuffersToDynamicMesh(LODResource.VertexBuffers, LODResource.IndexBuffer, LODResource.Sections, *Result);

			return Result;
		}

		// return true if the static mesh has a "hi-res" LOD ( e.g. source for nanite build)
		static bool HasHiResLOD(const UStaticMesh& StaticMesh)
		{
			return StaticMesh.IsHiResMeshDescriptionValid();
		}

		// return a dynamic mesh version of the high-res mesh if it exists (otherwise null )
		static TSharedPtr<FDynamicMesh3> GetHiResLOD(const UStaticMesh& StaticMesh)
		{
			TSharedPtr<FDynamicMesh3> Result;

			if (HasHiResLOD(StaticMesh))
			{

				if (FMeshDescription* HiResMeshDescription = StaticMesh.GetHiResMeshDescription())
				{
					// TODO: we only need to do this copy if normals or tangents are set to auto-generated...
					FMeshBuildSettings HiResBuildSettings = StaticMesh.GetHiResSourceModel().BuildSettings;
					FMeshDescription TmpMeshDescription(*HiResMeshDescription);
					UE::MeshDescription::InitializeAutoGeneratedAttributes(TmpMeshDescription, &HiResBuildSettings);
					{
						Result = MakeShared<FDynamicMesh3>();
						FMeshDescriptionToDynamicMesh Converter;
						Converter.Convert(&TmpMeshDescription, *Result, true);
					}
				
				}
			}
			return Result;
		}

		// return a dynamic mesh version of the high-res mesh converted to render data ( if it exists, otherwise null )
		static TSharedPtr<FDynamicMesh3> GetHiResLODAsRenderData(const UStaticMesh& StaticMesh, int32& VertexCount, int32& TriangleCount)
		{
			VertexCount = -1;
			TriangleCount = -1;

			TSharedPtr<FDynamicMesh3> Result;

			if (HasHiResLOD(StaticMesh))
			{
				if (FMeshDescription* HiResMeshDescription = StaticMesh.GetHiResMeshDescription())
				{
					UStaticMesh* TmpStaticMesh = NewObject<UStaticMesh>(GetTransientPackage());
					TmpStaticMesh->AddSourceModel();
					if (FMeshDescription* LOD0MeshDescription = TmpStaticMesh->CreateMeshDescription(0))
					{
						*LOD0MeshDescription = *HiResMeshDescription;
						TmpStaticMesh->CommitMeshDescription(0);
						TmpStaticMesh->PostEditChange(); // trigger build of the render data.

						Result = MakeShared<FDynamicMesh3>();
						const FStaticMeshRenderData* RenderData = TmpStaticMesh->GetRenderData();
						const FStaticMeshLODResources& LODResource = RenderData->LODResources[0];
						
						VertexCount = LODResource.VertexBuffers.PositionVertexBuffer.GetNumVertices();
						TriangleCount = LODResource.IndexBuffer.GetNumIndices() / 3;

						UE::Conversion::RenderBuffersToDynamicMesh(LODResource.VertexBuffers, LODResource.IndexBuffer, LODResource.Sections, *Result);
					}
				}
			}

			return Result;
		}
	};


	// Cache's FDynamicMesh3 representations of the imported LODs and RenderData LODs (for display use in the tool)
	// as well as the triangle and vertex counts for the StaticMesh LODs.
	class FDynamicMeshLODCache
	{
	public:

		struct FMeshElementInfo
		{
			int32 VertexCount = -1;
			int32 TriangleCount = -1;
			bool IsValid() const {return !(VertexCount == -1 || TriangleCount == -1);}
		};
		
		
		FDynamicMeshLODCache(UStaticMesh& StaticMeshIn)
		: StaticMesh(&StaticMeshIn)
		{
			MaxLOD = StaticMesh->GetNumSourceModels();
			ResetLODMeshInfo();
		}

		// compute and cache the triangle and vertex counts for the original source meshes stored in the UStaticMesh.
		void ResetLODMeshInfo()
		{
			LODToMeshInfo.Reset();
			RenderLODToMeshInfo.Reset();


			int32 NumSourceModels = StaticMesh->GetNumSourceModels();
			for (int32 si = 0; si < NumSourceModels; ++si)
			{
				
				if (FMeshDescription* LODMesh = StaticMesh->GetMeshDescription(si))		// generated LODs do not have mesh description...is that the only way to tell?
				{
					FMeshElementInfo LODMeshInfo = { LODMesh->Vertices().Num(), LODMesh->Triangles().Num() };
					
					TTuple<int32, FMeshElementInfo> KeyValuePair = {si, LODMeshInfo};
					LODToMeshInfo.Add(KeyValuePair);
				}
			}


			if (StaticMesh->HasValidRenderData())
			{
				const FStaticMeshRenderData* RenderData = StaticMesh->GetRenderData();
				int NumRenderLODs = RenderData->LODResources.Num();
				for (int32 ri = 0; ri < NumRenderLODs; ++ri)
				{
					const FStaticMeshLODResources& LODResource = RenderData->LODResources[ri];
					
					FMeshElementInfo LODMeshInfo;
					LODMeshInfo.VertexCount = LODResource.VertexBuffers.PositionVertexBuffer.GetNumVertices();
					LODMeshInfo.TriangleCount = LODResource.IndexBuffer.GetNumIndices() / 3;
					TTuple<int32, FMeshElementInfo> KeyValuePair = { ri, LODMeshInfo };
					RenderLODToMeshInfo.Add(KeyValuePair);
				}
			}

			if (HasHiResLOD())
			{
				const FMeshDescription* HiResMesh = StaticMesh->GetHiResMeshDescription();
				HiResMeshInfo = { HiResMesh->Vertices().Num(), HiResMesh->Triangles().Num() };
			}
		}



		// -- Imported LODs --
		bool HasImportedLOD(int32 LODIndex) const
		{
			return FMeshLODUtils::HasImportedLOD(*StaticMesh, LODIndex);
		}

		FMeshElementInfo GetLODMeshElementInfo(int32 LODIndex) const
		{
			if (LODToMeshInfo.Contains(LODIndex))
			{
				return LODToMeshInfo[LODIndex];
			}
			return  FMeshElementInfo();
		}

		// get DynamicMesh representation of the imported LOD.
		// returns null ptr if the the LOD was not an imported mesh.
		TSharedPtr<FDynamicMesh3> GetImportedLOD(int32 LODIndex)
		{
			TSharedPtr<FDynamicMesh3> Result;
			if (!HasImportedLOD(LODIndex))
			{
				return Result;
			}

			if (!LODToDynamicMesh.Contains(LODIndex))
			{

				Result = FMeshLODUtils::GetImportedLOD(*StaticMesh, LODIndex);
				if (Result.IsValid())
				{
					LODToDynamicMesh.Add(LODIndex, Result);
				}				
			}

			if (LODToDynamicMesh.Contains(LODIndex))
			{
				Result = LODToDynamicMesh[LODIndex];
			}

			return Result;
		}

		// -- Imported HiRes Mesh
		bool HasHiResLOD() const
		{
			return FMeshLODUtils::HasHiResLOD(*StaticMesh);
		}
		// get triangle and vertex count for hi-res mesh. Returns invalid Info if no hi-res mesh.
		const FMeshElementInfo& GetHiResMeshInfo() const
		{
			return HiResMeshInfo;
		}
		// return DynamiMesh representation of the HiRes mesh if it exists, otherwise an invalid SharedPtr
		TSharedPtr<FDynamicMesh3> GetHiResLOD()
		{
			if (!HiResDynamicMesh)
			{
				HiResDynamicMesh = FMeshLODUtils::GetHiResLOD(*StaticMesh);
			}

			return HiResDynamicMesh;
		}

		// -- Render LODs
		bool HasRenderLOD(int32 LODIndex) const
		{
			return FMeshLODUtils::HasRenderLOD(*StaticMesh, LODIndex);
		}

		// get triangle and vertex count for specified RenderData mesh if it exists, otherwise an invalid info.
		FMeshElementInfo GetRenderLODMeshInfo(int32 LODIndex) const
		{
			if (RenderLODToMeshInfo.Contains(LODIndex))
			{
				return RenderLODToMeshInfo[LODIndex];
			}

			return FMeshElementInfo();
		}

		// return DynamicMesh representation of the specified RenderData LOD if it exists, otherwise an invalid SharedPtr
		TSharedPtr<FDynamicMesh3> GetRenderLOD(int32 LODIndex)
		{
			TSharedPtr<FDynamicMesh3> Result;
			
			if (!HasRenderLOD(LODIndex))
			{
				return Result;
			}

			if (!RenderLODToDynamicMesh.Contains(LODIndex))
			{
				Result = FMeshLODUtils::GetRenderLOD(*StaticMesh, LODIndex);
				
				if (Result.IsValid())
				{
					RenderLODToDynamicMesh.Add(LODIndex, Result);
				}
			}

			if (RenderLODToDynamicMesh.Contains(LODIndex))
			{
				Result = RenderLODToDynamicMesh[LODIndex];
			}

			return Result;
		}

		// returns DynamicMesh representation of the HiRes Mesh after it is converted to a RenderData form. 
		// will return invalid SharedPtr if no hi-res mesh exists.
		TSharedPtr<FDynamicMesh3> GetHiResLODAsRenderData()
		{
			if (!HiResRenderDynamicMesh)
			{
				HiResRenderDynamicMesh = FMeshLODUtils::GetHiResLODAsRenderData(*StaticMesh, HiResRenderMeshInfo.VertexCount, HiResRenderMeshInfo.TriangleCount);
			}

			return HiResRenderDynamicMesh;
		}
		// returns the triangle and vertex count of the render data representation of the hi-res mesh. 
		// will return invalid info if no hi-res exists.
		FMeshElementInfo GetHiResLODAsRenderDataMeshInfo()
		{
			if (!HiResRenderMeshInfo.IsValid() && HasHiResLOD())
			{
				// trigger build and cache of hi-res as renderdata, this captures the needed info
				GetHiResLODAsRenderData();
			}

			return HiResRenderMeshInfo;
		}

	protected:

		int32 MaxLOD;
		// caches for imported LODs
		TMap<int32, FMeshElementInfo>  LODToMeshInfo;
		TMap<int32, TSharedPtr<FDynamicMesh3>> LODToDynamicMesh;
		
		// caches for render data LODs
		TMap<int32, FMeshElementInfo>  RenderLODToMeshInfo;
		TMap<int32, TSharedPtr<FDynamicMesh3>> RenderLODToDynamicMesh;
		
		// for imported hi-res
		FMeshElementInfo HiResMeshInfo;
		TSharedPtr<FDynamicMesh3> HiResDynamicMesh;

		// for render data hi-res 
		FMeshElementInfo HiResRenderMeshInfo;
		TSharedPtr<FDynamicMesh3> HiResRenderDynamicMesh;

		UStaticMesh* StaticMesh;
	};


	typedef ULODManagerTool::FLODMeshInfo  FLODMeshInfo;
	typedef ULODManagerTool::FLODName      FLODName;
	
	// proxy for information that the tool mutates, i.e. a mapping of LOD index to imported LODs and list of deleted materials
	struct FProxyLODState
	{
		// Identifies the imported LOD in the original static mesh.
		struct FImportedLODId
		{
			FImportedLODId() : SrcLODIndex(-2){}

			int32 SrcLODIndex = -2; // the index of this LOD in the source static mesh.  -1 is hi-res.

			bool IsValid() const { return SrcLODIndex > -2; }
			bool IsHighRes() const { return SrcLODIndex == -1;}
			void Reset() 
			{ 
				SrcLODIndex = -2;
			}

			bool operator==(const FImportedLODId& Other) const
			{
				return (SrcLODIndex == Other.SrcLODIndex);
			}
		};


		FProxyLODState() {}

		FProxyLODState(const UStaticMesh& StaticMesh)
		{
			Reset(StaticMesh);
		}

		// clear data
		void Reset()
		{
			HiRes.Reset();
			LODs.Reset();
		}

		// reset the imported lod mapping and clear the list of deleted materials
		void Reset(const UStaticMesh& StaticMesh)
		{
			Reset();

			if (StaticMesh.IsHiResMeshDescriptionValid())
			{
				HiRes.SrcLODIndex = -1;
			}

			int32 NumSourceModels = StaticMesh.GetNumSourceModels();
			LODs.SetNum(NumSourceModels);
			for (int i = 0; i < NumSourceModels; ++i)
			{
				EMeshLODIdentifier LOD = (i < 8) ? static_cast<EMeshLODIdentifier>(i) : EMeshLODIdentifier::LOD7;
				LODs[i].SrcLODIndex = i;
			}
		}

		void DeleteHiResSourceModel()
		{
			HiRes.Reset();
		}

		void MoveHiResToLOD0()
		{
			if (HiRes.IsValid())
			{
				LODs[0] = HiRes;
				HiRes.Reset();
			}
		}

		bool HasHiResAsLOD0() const
		{
			return !HiRes.IsValid() && LODs.IsValidIndex(0) && LODs[0].IsHighRes();
		}

		// return true if there are no materials to delete and the imported LOD order matches the static mesh 
		//( i.e. no changes are needed to Sync the static mesh )
		bool IsInSync(const UStaticMesh& StaticMesh) const
		{
			if (DeletedMaterialIndices.IsEmpty())
			{
				if (StaticMesh.IsHiResMeshDescriptionValid() == HiRes.IsValid())
				{
					bool IsSame = true;
					int32 NumSourceModels = StaticMesh.GetNumSourceModels();
					for (int i = 0; i < NumSourceModels; ++i)
					{
						EMeshLODIdentifier LOD = (i < 8) ? static_cast<EMeshLODIdentifier>(i) : EMeshLODIdentifier::LOD7;
						IsSame = IsSame && (LODs[i].SrcLODIndex == i);
					}
					return IsSame;
				}
			}
			return false;
 		}


		// update the static mesh to match the managed state.  
		// Note, this calls modify on the mesh and should happen inside a transaction!
		bool ModifyStaticMesh(UStaticMesh& StaticMesh)
		{
			if (IsInSync(StaticMesh)) // nothing to be done.
			{
				return false;
			}

			const bool bDeleteMaterials = ( DeletedMaterialIndices.Num() > 0 );
			bool bUpdatedLOD0 = false;

			TArray<FStaticMaterial> NewMaterialSet;
			TMap<FPolygonGroupID, FPolygonGroupID> RemapPolygonGroups;
			
			if (bDeleteMaterials) // compute NewMaterialSet and RemapPolygonGroups
			{
				const TArray<FStaticMaterial>& CurrentMaterialSet = StaticMesh.GetStaticMaterials();
				const int32 NumMaterials = CurrentMaterialSet.Num();
				TArray<int32> MaterialIDMap;
				MaterialIDMap.Init(-1, NumMaterials);
				for (int32 k = 0; k < NumMaterials; ++k)
				{
					if (!DeletedMaterialIndices.Contains(k))
					{
						MaterialIDMap[k] = NewMaterialSet.Num();
						RemapPolygonGroups.Add(FPolygonGroupID(k), FPolygonGroupID(MaterialIDMap[k]));
						NewMaterialSet.Add(CurrentMaterialSet[k]);
					}
				}
			}

			StaticMesh.Modify();

			// was the high res moved or deleted?
			if (StaticMesh.IsHiResMeshDescriptionValid() && !HiRes.IsValid())
			{

				StaticMesh.ModifyHiResMeshDescription();

				bool bHiResMoved = (LODs[0].IsHighRes());

				if (bHiResMoved)
				{
					FMeshDescription* HiResMeshDescription = StaticMesh.GetHiResMeshDescription();

					StaticMesh.ModifyMeshDescription(0);
					FMeshDescription* LOD0MeshDescription = StaticMesh.GetMeshDescription(0);
					*LOD0MeshDescription = *HiResMeshDescription;
					if (bDeleteMaterials)
					{
						LOD0MeshDescription->RemapPolygonGroups(RemapPolygonGroups);
					}
					StaticMesh.CommitMeshDescription(0);

					bUpdatedLOD0 = true;
				}
			
				StaticMesh.ClearHiResMeshDescription();
				StaticMesh.CommitHiResMeshDescription();
				
			}

			if (bDeleteMaterials)
			{
				// update the material set
				StaticMesh.GetStaticMaterials() = NewMaterialSet;

				// remap the polygon groups in the retained mesh descriptions
				{
					if (HiRes.IsValid())
					{
						StaticMesh.ModifyHiResMeshDescription();
						FMeshDescription* HiResMeshDescription = StaticMesh.GetHiResMeshDescription();
						HiResMeshDescription->RemapPolygonGroups(RemapPolygonGroups);
						StaticMesh.CommitHiResMeshDescription();
					}

					const int32 kStart = (bUpdatedLOD0 == false) ? 0 : 1;


					for (int32 k = kStart; k < StaticMesh.GetNumSourceModels(); ++k)
					{
						if (StaticMesh.IsSourceModelValid(k))
						{
							if (FMeshDescription* Mesh = StaticMesh.GetMeshDescription(k))
							{
								StaticMesh.ModifyMeshDescription(k);
								Mesh->RemapPolygonGroups(RemapPolygonGroups);
								StaticMesh.CommitMeshDescription(k);
							}
						}
					}
				}
			}

			return true;


		}

		// information this tools tracks
		FImportedLODId HiRes;
		TArray< FImportedLODId > LODs;  // map LOD index to imported LODs.
		TSet<int32> DeletedMaterialIndices;

	};


	/**
	* FLODManagerToolChange represents an reversible change to ProxyLODState
	* ULODManagerTool is the only supported target.
	*/
	class FLODManagerToolChange : public FToolCommandChange
	{
	public:
		
		FLODManagerToolChange(const FText& ChangeDescription)
		:Description(ChangeDescription)
		{}

		FProxyLODState OldMeshLODState;
		FProxyLODState NewMeshLODState;

		/** Makes the change to the object */
		virtual void Apply(UObject* Object) override 
		{
			ULODManagerTool* ChangeTarget = CastChecked<ULODManagerTool>(Object);
			ChangeTarget->ApplyChange(this, false);
		}

		/** Reverts change to the object */
		virtual void Revert(UObject* Object) override
		{
			ULODManagerTool* ChangeTarget = CastChecked<ULODManagerTool>(Object);
			ChangeTarget->ApplyChange(this, true);
		}

		/** Describes this change (for debugging) */
		virtual FString ToString() const override
		{
			return (Description.IsEmpty()) ? FString(TEXT("LODManager Change")) : Description.ToString();
		}

		FText Description;	
	};

}}}



ULODManagerTool::ULODManagerTool()
{
}


void ULODManagerTool::Setup()
{
	UInteractiveTool::Setup();

	if (! ensure(Targets.Num() == 1) )
	{
		return;
	}

	UStaticMesh* StaticMesh = nullptr;
	if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(UE::ToolTarget::GetTargetComponent(Targets[0])))
	{
		StaticMesh = StaticMeshComponent->GetStaticMesh();
		if (StaticMesh)
		{ 
			UE::ToolTarget::HideSourceObject(Targets[0]);

			ProxyLODState = MakeUnique<UE::Geometry::LODManagerHelper::FProxyLODState>(*StaticMesh);
			DynamicMeshCache = MakeUnique<UE::Geometry::LODManagerHelper::FDynamicMeshLODCache>(*StaticMesh);
		}
	}

	const bool bHasHiRes = (StaticMesh) ? StaticMesh->IsHiResMeshDescriptionValid() : false;
	LODPreviewProperties = NewObject<ULODManagerPreviewLODProperties>(this);
	AddToolPropertySource(LODPreviewProperties);
	
	UpdateLODNames();
	const int32 DefaultNameIndex = (bHasHiRes) ? 1 : 0; // want the index of the first LOD name, not the hi-res
	
	LODPreviewProperties->VisibleLOD = LODPreviewProperties->LODNamesList[DefaultNameIndex];
	LODPreviewProperties->WatchProperty(LODPreviewProperties->VisibleLOD, [this](FString NewLOD) { bPreviewLODValid = false; });
	LODPreviewProperties->WatchProperty(LODPreviewProperties->bShowSeams, [this](bool bNewValue) { bPreviewLODValid = false; });

	LODPreview = NewObject<UPreviewMesh>(this);
	LODPreview->CreateInWorld(GetTargetWorld(), (FTransform)UE::ToolTarget::GetLocalToWorldTransform(Targets[0]));
	LODPreview->SetTangentsMode(EDynamicMeshComponentTangentsMode::ExternallyProvided);
	LODPreview->SetVisible(false);

	LODPreviewLines = NewObject<UPreviewGeometry>(this);
	LODPreviewLines->CreateInWorld(GetTargetWorld(), (FTransform)UE::ToolTarget::GetLocalToWorldTransform(Targets[0]));

	FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Targets[0]);
	LODPreview->SetMaterials(MaterialSet.Materials);

	bLODInfoValid = false;

	// Optionally add buttons to delete or move the hi-res mesh
	if (bHasHiRes)
	{
		HiResSourceModelActions = NewObject<ULODManagerHiResSourceModelActions>(this);
		HiResSourceModelActions->Initialize(this);
		AddToolPropertySource(HiResSourceModelActions);
	}
	

	MaterialActions = NewObject<ULODManagerMaterialActions>(this);
	MaterialActions->Initialize(this);
	AddToolPropertySource(MaterialActions);

	LODInfoProperties = NewObject<ULODManagerLODProperties>(this);
	AddToolPropertySource(LODInfoProperties);
	bLODInfoValid = false;

	SetToolDisplayName(LOCTEXT("ToolName", "Manage LODs"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool", "Inspect and Modify LODs of a StaticMesh Asset"),
		EToolMessageLevel::UserNotification);
}


void ULODManagerTool::ApplyChange(const UE::Geometry::LODManagerHelper::FLODManagerToolChange* Change, bool bRevert)
{
	check(ProxyLODState)
	if (!bRevert)
	{
		*ProxyLODState = Change->NewMeshLODState;
	}
	else
	{
		*ProxyLODState = Change->OldMeshLODState;
	}
}

// copy current state into the active change as Old
void ULODManagerTool::BeginChange(FText TransactionName)
{
	if(!ActiveChange.IsValid())
	{ 
		ActiveChange = MakeUnique< UE::Geometry::LODManagerHelper::FLODManagerToolChange >(TransactionName);
		ActiveChange->OldMeshLODState = *ProxyLODState;
	}
}

// copy current state into the active change as New and emit change object
void ULODManagerTool::EndChange()
{
	if (ActiveChange.IsValid())
	{
		ActiveChange->NewMeshLODState= *ProxyLODState;
		const FText& TransactionName = ActiveChange->Description;
		GetToolManager()->EmitObjectChange(this, MoveTemp(ActiveChange), TransactionName);
	}
}

void ULODManagerTool::OnShutdown(EToolShutdownType ShutdownType)
{
	if (LODPreview)
	{
		LODPreview->Disconnect();
		LODPreviewLines->Disconnect();
	}
	// restore visibility
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		UE::ToolTarget::ShowSourceObject(Targets[ComponentIdx]);
	}

	if (UStaticMesh* StaticMesh = GetSingleStaticMesh())
	{ 
		if (ShutdownType == EToolShutdownType::Accept)
		{ 
			// the regular transaction 
			GetToolManager()->BeginUndoTransaction(LOCTEXT("AcceptLODManager", "Accept LOD Manager"));
			bool bModifiedMesh = ProxyLODState->ModifyStaticMesh(*StaticMesh);
		
			// @todo consider moving PostEditChange to outside the scope of the transaction. 
			if (bModifiedMesh)
			{
				StaticMesh->PostEditChange();
			}
			GetToolManager()->EndUndoTransaction();
		}
	}
}



void ULODManagerTool::RequestAction(ELODManagerToolActions ActionType)
{
	if (PendingAction == ELODManagerToolActions::NoAction)
	{
		PendingAction = ActionType;
	}
}



void ULODManagerTool::OnTick(float DeltaTime)
{
	switch (PendingAction)
	{
	case ELODManagerToolActions::DeleteHiResSourceModel:
		DeleteHiResSourceModel();
		break;

	case ELODManagerToolActions::MoveHiResToLOD0:
		MoveHiResToLOD0();
		break;

	case ELODManagerToolActions::RemoveUnreferencedMaterials:
		RemoveUnreferencedMaterials();
		break;
	}
	PendingAction = ELODManagerToolActions::NoAction;

	// test capture a dirty state that results from undo while in the tool.
	bool IsLODInfoDirty = [&, this]
		{
			UStaticMesh* StaticMesh = GetSingleStaticMesh();
			if (!StaticMesh || !LODInfoProperties || !ProxyLODState)
			{
				return false;
			}

			// have we changed the number of materials
			const int32 NumDeletedMaterials =  ProxyLODState->DeletedMaterialIndices.Num();
			const int32 CurrentNumMaterial = StaticMesh->GetStaticMaterials().Num() - NumDeletedMaterials;
			const int32 DisplayedNumMaterials = LODInfoProperties->Materials.Num();
			
			bool bDirty = (DisplayedNumMaterials != CurrentNumMaterial);

			// what about hi-res mesh?
			bDirty = bDirty || (ProxyLODState->HiRes.IsValid() ? LODInfoProperties->HiResSource.Num() == 0 : LODInfoProperties->HiResSource.Num() != 0);

			return bDirty;
		}();

	bLODInfoValid = bLODInfoValid && !IsLODInfoDirty;

	if (bLODInfoValid == false)
	{
		UpdateLODInfo();
	}

	if (bPreviewLODValid == false)
	{
		UpdatePreviewLOD();
	}
}



UStaticMesh* ULODManagerTool::GetSingleStaticMesh()
{
	if (Targets.Num() > 1)
	{
		return nullptr;
	}
	UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(UE::ToolTarget::GetTargetComponent(Targets[0]));
	
	if (StaticMeshComponent == nullptr || StaticMeshComponent->GetStaticMesh() == nullptr)
	{
		return nullptr;
	}
	UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
	return StaticMesh;
}







void ULODManagerTool::UpdateLODInfo()
{
	using namespace UE::Geometry::LODManagerHelper;

	UStaticMesh* StaticMesh = GetSingleStaticMesh();
	if (bLODInfoValid == true || StaticMesh == nullptr)
	{
		return;
	}
	bLODInfoValid = true;

	// update the displayed info about the LODS
	LODInfoProperties->bNaniteEnabled = StaticMesh->IsNaniteEnabled();
	LODInfoProperties->KeepTrianglePercent = StaticMesh->GetNaniteSettings().KeepPercentTriangles * 100;

	const TArray<FStaticMaterial>& OriginalMaterialSet = StaticMesh->GetStaticMaterials();
	
	// filter the original material set, against those pending delete, into the displayed materials (LODInfoProperties)
	LODInfoProperties->Materials.Reset();	
	for (int32 mid = 0, NumMaterials =  OriginalMaterialSet.Num(); mid < NumMaterials; ++mid )
	{
		if (!ProxyLODState->DeletedMaterialIndices.Contains(mid))
		{
			LODInfoProperties->Materials.Add(OriginalMaterialSet[mid]);
		}
	}


	// per-LOD info
	LODInfoProperties->SourceLODs.Reset();
	for (const FProxyLODState::FImportedLODId& ImportedLODInfo : ProxyLODState->LODs)
	{
		if (ImportedLODInfo.IsValid())
		{
			const FDynamicMeshLODCache::FMeshElementInfo ElementCounts = (ImportedLODInfo.IsHighRes()) ? DynamicMeshCache->GetHiResMeshInfo() : DynamicMeshCache->GetLODMeshElementInfo(ImportedLODInfo.SrcLODIndex);
			const FLODManagerLODInfo LODInfo = { ElementCounts.VertexCount, ElementCounts.TriangleCount };
			LODInfoProperties->SourceLODs.Add(LODInfo);
		}
	}

	// hi-res source info
	LODInfoProperties->HiResSource.Reset();
	if (ProxyLODState->HiRes.IsValid() && ProxyLODState->HiRes.IsHighRes())
	{
		const FDynamicMeshLODCache::FMeshElementInfo& ElementCounts = DynamicMeshCache->GetHiResMeshInfo();
		const FLODManagerLODInfo LODInfo = { ElementCounts.VertexCount, ElementCounts.TriangleCount };
		LODInfoProperties->HiResSource.Add(LODInfo);
	}

	// render data lod info
	LODInfoProperties->RenderLODs.Reset();
	const bool bHiResInLOD0 = ProxyLODState->LODs.IsValidIndex(0) && ProxyLODState->LODs[0].IsHighRes();
	if (StaticMesh->HasValidRenderData())
	{
		const FStaticMeshRenderData* RenderData = StaticMesh->GetRenderData();
		const int32 NumRenderLODs = RenderData->LODResources.Num();

		if (bHiResInLOD0)
		{
			const FDynamicMeshLODCache::FMeshElementInfo ElementCounts = DynamicMeshCache->GetHiResLODAsRenderDataMeshInfo();
			FLODManagerLODInfo LODInfo = {ElementCounts.VertexCount, ElementCounts.TriangleCount};
			LODInfoProperties->RenderLODs.Add(LODInfo);
		}
		

		for (int32 i = (bHiResInLOD0)? 1: 0 ; i < NumRenderLODs; ++i)
		{ 
			if (DynamicMeshCache->HasRenderLOD(i))
			{
				const FDynamicMeshLODCache::FMeshElementInfo ElementCounts = DynamicMeshCache->GetRenderLODMeshInfo(i);
				FLODManagerLODInfo LODInfo = { ElementCounts.VertexCount, ElementCounts.TriangleCount };
			}
		}
	}


	UpdateLODNames();

	bPreviewLODValid = false;   
}



void ULODManagerTool::UpdateLODNames()
{
	UStaticMesh* StaticMesh = GetSingleStaticMesh();
	if (StaticMesh == nullptr || LODPreviewProperties == nullptr)
	{
		return;
	}

	TArray<FString>& UILODNames = LODPreviewProperties->LODNamesList;
	UILODNames.Reset();
	ActiveLODNames.Reset();

	// Hi-Res 
	if (ProxyLODState->HiRes.IsValid())
	{
		FString LODName(TEXT("SourceModel HiRes"));
		UILODNames.Add(LODName);
		ActiveLODNames.Add(LODName, { -1, -1, 0 });
	}

	// Standard LODs
	int32 NumSourceModels = ProxyLODState->LODs.Num();
	for (int32 si = 0; si < NumSourceModels; ++si)
	{ 
		const UE::Geometry::LODManagerHelper::FProxyLODState::FImportedLODId& LODInfo = ProxyLODState->LODs[si];
	
		if (LODInfo.IsValid())
		{
			FString LODUIName = FString::Printf(TEXT("SourceModel LOD%d"), si);
			UILODNames.Add(LODUIName);

			if (LODInfo.IsHighRes()) // test if this is a moved hi-res
			{
				ActiveLODNames.Add(LODUIName, { -1, -1, 0 });
			}
			else
			{
				ActiveLODNames.Add(LODUIName, { LODInfo.SrcLODIndex, -1, -1 });
			}
		}
	}

	const bool bHiResAtLOD0 = ProxyLODState->HasHiResAsLOD0();

	if (StaticMesh->HasValidRenderData())
	{
		const FStaticMeshRenderData* RenderData = StaticMesh->GetRenderData();
		int32 Index = 0;
		for (const FStaticMeshLODResources& LODResource : RenderData->LODResources)
		{
			FString LODUIName = FString::Printf(TEXT("RenderData LOD%d"), Index);
			UILODNames.Add(LODUIName);
			if (Index == 0 && bHiResAtLOD0) // test if we need render data for moved hi-res at lod 0
			{
				ActiveLODNames.Add(LODUIName, { -1, -1, 1 }); // render data version of original hi-res
			}
			else
			{
				ActiveLODNames.Add(LODUIName, { -1, Index, -1 });
			}
			Index++;
		}
	}

	// update the name selected in the UI if needed.
	if (!ActiveLODNames.Contains(LODPreviewProperties->VisibleLOD))
	{
		LODPreviewProperties->VisibleLOD = UILODNames[0];
	}
}





void ULODManagerTool::UpdatePreviewLines(FLODMeshInfo& MeshInfo)
{
	
	float LineWidthMultiplier = 1.0f;
	FColor BoundaryEdgeColor(240, 15, 15);
	float BoundaryEdgeThickness = LineWidthMultiplier * 4.0f;
	float BoundaryEdgeDepthBias = 2.0f;

	ULineSetComponent* BoundaryLines = LODPreviewLines->FindLineSet(TEXT("BoundaryLines"));
	if (BoundaryLines == nullptr)
	{
		BoundaryLines = LODPreviewLines->AddLineSet(TEXT("BoundaryLines"));
	}
	if (BoundaryLines)
	{
		BoundaryLines->Clear();
		for (int32 eid : MeshInfo.BoundaryEdges)
		{
			FVector3d A, B;
			MeshInfo.Mesh->GetEdgeV(eid, A, B);
			BoundaryLines->AddLine((FVector)A, (FVector)B, BoundaryEdgeColor, BoundaryEdgeThickness, BoundaryEdgeDepthBias);
		}
	}
}


void ULODManagerTool::ClearPreviewLines()
{
	ULineSetComponent* BoundaryLines = LODPreviewLines->FindLineSet(TEXT("BoundaryLines"));
	if (BoundaryLines)
	{
		BoundaryLines->Clear();
	}
}


void ULODManagerTool::UpdatePreviewLOD()
{
	UStaticMesh* StaticMesh = GetSingleStaticMesh();
	if (StaticMesh == nullptr || LODPreviewProperties == nullptr || bPreviewLODValid == true)
	{
		return;
	}
	bPreviewLODValid = true;


	// the UI mesh selection
	const FString SelectedLOD = LODPreviewProperties->VisibleLOD;

	// Get an identifier for the src mesh
	if (const FLODName* FoundName = ActiveLODNames.Find(SelectedLOD))
	{
		TUniquePtr<FLODMeshInfo> MeshInfo = GetLODMeshInfo(*FoundName);
		
		if (MeshInfo->Mesh.IsValid())
		{ 
			LODPreview->ReplaceMesh(*MeshInfo->Mesh);
			LODPreview->SetVisible(true);
			UpdatePreviewLines(*MeshInfo);
			LODPreviewLines->SetAllVisible(LODPreviewProperties->bShowSeams);
		}
	}
	
}




TUniquePtr<ULODManagerTool::FLODMeshInfo> ULODManagerTool::GetLODMeshInfo(const FLODName& LODName)
{
	TSharedPtr<FDynamicMesh3> AsDynamicMesh;

	if (LODName.OtherIndex == 0) // HiResSourceModel
	{
		AsDynamicMesh = DynamicMeshCache->GetHiResLOD();
	}
	else if (LODName.OtherIndex == 1) // Artificial version of the renderdata for the hi-res
	{
		AsDynamicMesh = DynamicMeshCache->GetHiResLODAsRenderData();
	}
	else if (LODName.SourceModelIndex >= 0) // requesting a source model
	{
		AsDynamicMesh = DynamicMeshCache->GetImportedLOD(LODName.SourceModelIndex);
	}
	else if (LODName.RenderDataIndex >= 0) // requesting render data
	{
		AsDynamicMesh = DynamicMeshCache->GetRenderLOD(LODName.RenderDataIndex);
	}

	TUniquePtr<FLODMeshInfo> NewMeshInfo = MakeUnique<FLODMeshInfo>();
	if (AsDynamicMesh)
	{	
		NewMeshInfo->Mesh = AsDynamicMesh; 
		for (int32 eid : NewMeshInfo->Mesh->EdgeIndicesItr())
		{
			if (NewMeshInfo->Mesh->IsBoundaryEdge(eid))
			{
				NewMeshInfo->BoundaryEdges.Add(eid);
			}
		}
	}
	return MoveTemp(NewMeshInfo);
}


void ULODManagerTool::DeleteHiResSourceModel()
{
	BeginChange(LOCTEXT("LODManagerDeleteHiRes", "Delete HiRes"));
	ProxyLODState->DeleteHiResSourceModel();
	EndChange();

	bLODInfoValid = false;
}


void ULODManagerTool::MoveHiResToLOD0()
{
	BeginChange(LOCTEXT("LODManagerMoveHiRes", "Move HiRes to LOD0"));
	ProxyLODState->MoveHiResToLOD0();
	EndChange();

	bLODInfoValid = false;
}


void ULODManagerTool::RemoveUnreferencedMaterials()
{
	
	const UStaticMesh* StaticMesh = GetSingleStaticMesh();

	// get the number of materials on the source static mesh.
	const int32 NumMaterials = GetSingleStaticMesh()->GetStaticMaterials().Num();

	// compute mask, indicates if a given index in original material set corresponds material that is acually being used.
	const TArray<bool> MatUsedFlags = [this, NumMaterials, StaticMesh]
		{
			TArray<bool> Used;
			Used.Init(false, NumMaterials);
			
			// review materials used on the mesh in the hi-res slot.
			if (ProxyLODState->HiRes.IsValid())
			{
				const int32 SrcLODIndex = ProxyLODState->HiRes.SrcLODIndex;

				EMeshLODIdentifier LODIdentifier = (SrcLODIndex == -1) ? EMeshLODIdentifier::HiResSource : static_cast<EMeshLODIdentifier>(SrcLODIndex);
				const TArray<int32> SectionToMaterialIndex = UStaticMeshToolTarget::MapSectionToMaterialID(StaticMesh, LODIdentifier);
				for (int32 MaterialIndex : SectionToMaterialIndex)
				{
					if (Used.IsValidIndex(MaterialIndex))
					{
						Used[MaterialIndex] = true;
					}
				}
			}

			// review materials in on the meshes in the regular lod array.
			for (const auto& LOD : ProxyLODState->LODs)
			{
				const int32 SrcLODIndex = LOD.SrcLODIndex;

				if (LOD.IsValid())
				{
					EMeshLODIdentifier LODIdentifier = (SrcLODIndex == -1) ? EMeshLODIdentifier::HiResSource : static_cast<EMeshLODIdentifier>(SrcLODIndex);
					const TArray<int32> SectionToMaterialIndex = UStaticMeshToolTarget::MapSectionToMaterialID(StaticMesh, LODIdentifier);
					for (int32 MaterialIndex : SectionToMaterialIndex)
					{
						if (Used.IsValidIndex(MaterialIndex))
						{
							Used[MaterialIndex] = true;
						}
					}
				}
			}
			return Used;
		}();



	BeginChange(LOCTEXT("LODManagerRemoveMaterials", "Delete Unused Materials"));

	// save list of unused (i.e. to be deleted) material indices 
	ProxyLODState->DeletedMaterialIndices.Reset();
	for (int32 i = 0; i < NumMaterials; ++i)
	{
		if (MatUsedFlags[i] == false)
		{
			ProxyLODState->DeletedMaterialIndices.Add(i);
		}
	}

	EndChange();

	bLODInfoValid = false;
}

#undef LOCTEXT_NAMESPACE

