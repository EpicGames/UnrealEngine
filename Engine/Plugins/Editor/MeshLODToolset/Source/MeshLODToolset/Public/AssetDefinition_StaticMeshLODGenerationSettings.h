// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "Tools/LODGenerationSettingsAsset.h"
#include "AssetDefinition_StaticMeshLODGenerationSettings.generated.h"

/**
 * Asset Definition for UStaticMeshLODGenerationSettings Assets
 */
UCLASS()
class UAssetDefinition_StaticMeshLODGenerationSettings : public UAssetDefinitionDefault
{
	GENERATED_BODY()

private:
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "FAssetTypeActions_StaticMeshLODGenerationSettings", "AutoLOD Settings"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(175, 0, 128)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UStaticMeshLODGenerationSettings::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Misc };
		return Categories;
	}
	virtual FAssetSupportResponse CanLocalize(const FAssetData& InAsset) const { return FAssetSupportResponse::NotSupported(); }
};

