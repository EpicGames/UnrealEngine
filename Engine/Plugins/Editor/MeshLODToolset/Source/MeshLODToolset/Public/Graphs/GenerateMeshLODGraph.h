// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryFlowGraph.h"
#include "DynamicMesh/MeshTangents.h"

#include "MeshProcessingNodes/MeshSolidifyNode.h"
#include "MeshProcessingNodes/MeshVoxMorphologyNode.h"
#include "MeshProcessingNodes/MeshMakeCleanGeometryNode.h"
#include "MeshProcessingNodes/MeshSimplifyNode.h"
#include "MeshProcessingNodes/MeshNormalsNodes.h"
#include "MeshProcessingNodes/MeshThickenNode.h"
#include "MeshProcessingNodes/MeshAutoGenerateUVsNode.h"
#include "MeshProcessingNodes/GenerateConvexHullMeshNode.h"
#include "DataTypes/MeshImageBakingData.h"
#include "PhysicsNodes/GenerateSimpleCollisionNode.h"

namespace UE::Geometry { template <typename PixelType> class TImageBuilder; }
namespace UE::GeometryFlow { struct FNormalMapImage; }
namespace UE::GeometryFlow { struct FTextureImage; }

using UE::Geometry::FDynamicMesh3;

struct FMeshLODGraphPreFilterSettings
{
	FName FilterGroupLayerName = FName();
};

class FGenerateMeshLODGraph
{
public:
	/**
	 * Initialize the LOD generation graph. 
	 * @param SourceMesh if the SourceMesh is known, some optimizations can be made in the graph, but this is optional
	 */
	void BuildGraph(const FDynamicMesh3* SourceMeshHint = nullptr);

	int32 AppendTextureBakeNode(const UE::Geometry::TImageBuilder<FVector4f>& SourceImage, const FString& Identifier);

	void AppendMultiTextureBakeNode(const TMap<int32, UE::GeometryFlow::TSafeSharedPtr<UE::Geometry::TImageBuilder<FVector4f>>>& SourceMaterialImages);

	void SetSourceMesh(const FDynamicMesh3& SourceMesh);


	void UpdatePreFilterSettings(const FMeshLODGraphPreFilterSettings& PreFilterSettings);
	const FMeshLODGraphPreFilterSettings& GetCurrentPreFilterSettings() const { return CurrentPreFilterSettings; }


	enum class ECoreMeshGeneratorMode
	{
		Solidify = 0,
		SolidifyAndClose = 1,
		SimplifyOnly = 2,
		ConvexHull = 3
	};
	void UpdateCoreMeshGeneratorMode(ECoreMeshGeneratorMode NewMode);
	ECoreMeshGeneratorMode GetCurrentCoreMeshGeneratorMode() const { return CurrentCoreMeshGeneratorMode; }

	void UpdateSolidifySettings(const FMeshSolidifySettings& SolidifySettings);
	const UE::GeometryFlow::FMeshSolidifySettings& GetCurrentSolidifySettings() const { return CurrentSolidifySettings; }

	void UpdateMorphologySettings(const UE::GeometryFlow::FVoxClosureSettings& MorphologySettings);
	const UE::GeometryFlow::FVoxClosureSettings& GetCurrentMorphologySettings() const { return CurrentMorphologySettings; }

	void UpdateMeshCleaningSettings(const UE::GeometryFlow::FMeshMakeCleanGeometrySettings& CleaningSettings);
	const UE::GeometryFlow::FMeshMakeCleanGeometrySettings& GetCurrentMeshCleaningSettings() const { return CurrentCleanMeshSettings; }

	void UpdateSimplifySettings(const UE::GeometryFlow::FMeshSimplifySettings& SimplifySettings);
	const UE::GeometryFlow::FMeshSimplifySettings& GetCurrentSimplifySettings() const { return CurrentSimplifySettings; }

	void UpdateGenerateConvexHullMeshSettings(const UE::GeometryFlow::FGenerateConvexHullMeshSettings& ConvexHullSettings);
	const UE::GeometryFlow::FGenerateConvexHullMeshSettings& GetCurrentGenerateConvexHullMeshSettings() const { return CurrentGenerateConvexHullMeshSettings; }

	void UpdateNormalsSettings(const UE::GeometryFlow::FMeshNormalsSettings& NormalsSettings);
	const UE::GeometryFlow::FMeshNormalsSettings& GetCurrentNormalsSettings() const { return CurrentNormalsSettings; }

	void UpdateAutoUVSettings(const UE::GeometryFlow::FMeshAutoGenerateUVsSettings& AutoUVSettings);
	const UE::GeometryFlow::FMeshAutoGenerateUVsSettings& GetCurrentAutoUVSettings() const { return CurrentAutoUVSettings; }

	void UpdateBakeCacheSettings(const UE::GeometryFlow::FMeshMakeBakingCacheSettings& BakeCacheSettings);
	const UE::GeometryFlow::FMeshMakeBakingCacheSettings& GetCurrentBakeCacheSettings() const { return CurrentBakeCacheSettings; }

	void UpdateGenerateSimpleCollisionSettings(const UE::GeometryFlow::FGenerateSimpleCollisionSettings& SimpleCollisionSettings);
	const UE::GeometryFlow::FGenerateSimpleCollisionSettings& GetCurrentGenerateSimpleCollisionSettings() const { return CurrentGenerateSimpleCollisionSettings; }

	void UpdateThickenWeightMap(const TArray<float>& ThickenWeightMap);

	void UpdateCollisionGroupLayerName(const FName& CollisionGroupLayerName);

	void UpdateThickenSettings(const UE::GeometryFlow::FMeshThickenSettings& ThickenSettings);
	const UE::GeometryFlow::FMeshThickenSettings& GetCurrentThickenSettings() const { return CurrentThickenSettings; }

	void EvaluateResult(
		FDynamicMesh3& ResultMesh,
		UE::Geometry::FMeshTangentsd& ResultTangents,
		UE::Geometry::FSimpleShapeSet3d& ResultCollision,
		UE::GeometryFlow::FNormalMapImage& NormalMap,
		TArray<TUniquePtr<UE::GeometryFlow::FTextureImage>>& TextureImages,
		UE::GeometryFlow::FTextureImage& MultiTextureImage,
		FProgressCancel* Progress);

protected:

	friend class UGenerateStaticMeshLODProcess;

	TUniquePtr<UE::GeometryFlow::FGraph> Graph;


	UE::GeometryFlow::FGraph::FHandle MeshSourceNode;

	UE::GeometryFlow::FGraph::FHandle FilterGroupsLayerNameNode;	// FNameSourceNode that defines name of polygroups layer
	UE::GeometryFlow::FGraph::FHandle FilterTrianglesNode;
	FMeshLODGraphPreFilterSettings CurrentPreFilterSettings;

	UE::GeometryFlow::FGraph::FHandle MeshGeneratorSwitchNode;
	ECoreMeshGeneratorMode CurrentCoreMeshGeneratorMode = ECoreMeshGeneratorMode::SolidifyAndClose;

	UE::GeometryFlow::FGraph::FHandle SolidifyNode;
	UE::GeometryFlow::FGraph::FHandle SolidifySettingsNode;
	UE::GeometryFlow::FMeshSolidifySettings CurrentSolidifySettings;

	UE::GeometryFlow::FGraph::FHandle MorphologyNode;
	UE::GeometryFlow::FGraph::FHandle MorphologySettingsNode;
	UE::GeometryFlow::FVoxClosureSettings CurrentMorphologySettings;

	UE::GeometryFlow::FGraph::FHandle CleanMeshNode;
	UE::GeometryFlow::FGraph::FHandle CleanMeshSettingsNode;
	UE::GeometryFlow::FMeshMakeCleanGeometrySettings CurrentCleanMeshSettings;

	UE::GeometryFlow::FGraph::FHandle SimplifyNode;
	UE::GeometryFlow::FGraph::FHandle SimplifySettingsNode;
	UE::GeometryFlow::FMeshSimplifySettings CurrentSimplifySettings;

	UE::GeometryFlow::FGraph::FHandle GenerateConvexHullMeshNode;
	UE::GeometryFlow::FGraph::FHandle GenerateConvexHullMeshSettingsNode;
	UE::GeometryFlow::FGenerateConvexHullMeshSettings CurrentGenerateConvexHullMeshSettings;

	UE::GeometryFlow::FGraph::FHandle NormalsNode;
	UE::GeometryFlow::FGraph::FHandle NormalsSettingsNode;
	UE::GeometryFlow::FMeshNormalsSettings CurrentNormalsSettings;

	UE::GeometryFlow::FGraph::FHandle AutoUVNode;
	UE::GeometryFlow::FGraph::FHandle AutoUVSettingsNode;
	UE::GeometryFlow::FMeshAutoGenerateUVsSettings CurrentAutoUVSettings;

	//UE::GeometryFlow::FGraph::FHandle RecomputeUVNode;
	//UE::GeometryFlow::FGraph::FHandle RecomputeUVSettingsNode;

	UE::GeometryFlow::FGraph::FHandle RepackUVNode;
	UE::GeometryFlow::FGraph::FHandle RepackUVSettingsNode;

	UE::GeometryFlow::FGraph::FHandle TangentsNode;
	UE::GeometryFlow::FGraph::FHandle TangentsSettingsNode;

	UE::GeometryFlow::FGraph::FHandle BakeCacheNode;
	UE::GeometryFlow::FGraph::FHandle BakeCacheSettingsNode;
	UE::GeometryFlow::FMeshMakeBakingCacheSettings CurrentBakeCacheSettings;

	UE::GeometryFlow::FGraph::FHandle BakeNormalMapNode;
	UE::GeometryFlow::FGraph::FHandle BakeNormalMapSettingsNode;

	UE::GeometryFlow::FGraph::FHandle BakeMultiTextureNode;
	UE::GeometryFlow::FGraph::FHandle BakeMultiTextureSettingsNode;
	UE::GeometryFlow::FGraph::FHandle MaterialIDTextureSourceNode;

	UE::GeometryFlow::FGraph::FHandle ThickenNode;
	UE::GeometryFlow::FGraph::FHandle ThickenSettingsNode;
	UE::GeometryFlow::FGraph::FHandle ThickenWeightMapNode;
	UE::GeometryFlow::FMeshThickenSettings CurrentThickenSettings;

	UE::GeometryFlow::FGraph::FHandle GroupLayerNameNode;

	struct FBakeTextureGraphInfo
	{
		int32 Index;
		FString Identifier;
		UE::GeometryFlow::FGraph::FHandle TexSourceNode;
		UE::GeometryFlow::FGraph::FHandle BakeNode;
	};
	TArray<FBakeTextureGraphInfo> BakeTextureNodes;

	UE::GeometryFlow::FGraph::FHandle DecomposeMeshForCollisionNode;
	
	UE::GeometryFlow::FGraph::FHandle GenerateSimpleCollisionNode;
	UE::GeometryFlow::FGraph::FHandle GenerateSimpleCollisionSettingsNode;
	UE::GeometryFlow::FGenerateSimpleCollisionSettings CurrentGenerateSimpleCollisionSettings;

	UE::GeometryFlow::FGraph::FHandle CollisionOutputNode;
	UE::GeometryFlow::FGraph::FHandle MeshOutputNode;
	UE::GeometryFlow::FGraph::FHandle TangentsOutputNode;

	FName CollisionGroupLayerName = TEXT("Default");
	
};
