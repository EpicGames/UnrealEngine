// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataTypes/NormalMapData.h"
#include "Engine/StaticMesh.h"



#include "DataTypes/TextureImageData.h"
#include "DynamicMesh/MeshTangents.h"
#include "ShapeApproximation/SimpleShapeSet3.h"
#include "GenerateStaticMeshLODProcess.generated.h"

class FGenerateMeshLODGraph;


class UTexture2D;
class UMaterialInstanceConstant;
class UMaterialInstanceDynamic;
struct FMeshDescription;
using UE::Geometry::FDynamicMesh3;



UENUM()
enum class EGenerateStaticMeshLODProcess_MeshGeneratorModes : uint8
{
	//~NOTE: must keep in sync with FGenerateMeshLODGraph::ECoreMeshGeneratorMode

	/**
	 * Generate a mesh using Marching Cubes to remesh the input shape.
	 * Use Winding Numbers to determine what is inside
	 */
	Solidify = 0,

	/**
	 * Like Solidify, but Dilate and Contract the shape to delete small
	 * negative features (sharp inner corners, small holes) before remeshing
	 */
	SolidifyAndClose = 1,

	/** Generate a mesh by simplifying input Mesh attributes and filling small holes */
	CleanAndSimplify = 2,

	/** Generate a mesh by calculating the Convex Hull of the input shape */
	ConvexHull = 3
};


USTRUCT()
struct FGenerateStaticMeshLODProcessSettings
{
	GENERATED_BODY()

	/** Method used to generate the initial mesh for AutoLOD processing */
	UPROPERTY(EditAnywhere, Category = MeshGenerator, meta = (DisplayName="Mesh Generator"))
	EGenerateStaticMeshLODProcess_MeshGeneratorModes MeshGenerator = EGenerateStaticMeshLODProcess_MeshGeneratorModes::SolidifyAndClose;

	// Solidify settings

	/** Target number of voxels along the maximum dimension for Solidify operation */
	UPROPERTY(EditAnywhere, Category = Solidify, meta = (UIMin = "8", UIMax = "1024", ClampMin = "8", ClampMax = "1024", DisplayName="Voxel Resolution",
	EditConditionHides, EditCondition = "MeshGenerator == EGenerateStaticMeshLODProcess_MeshGeneratorModes::Solidify || MeshGenerator == EGenerateStaticMeshLODProcess_MeshGeneratorModes::SolidifyAndClose"))
	int SolidifyVoxelResolution = 128;

	/** Winding number threshold to determine what is considered inside the mesh during Solidify */
	UPROPERTY(EditAnywhere, Category = Solidify, AdvancedDisplay, meta = (UIMin = "0.1", UIMax = ".9", ClampMin = "-10", ClampMax = "10",
	EditConditionHides, EditCondition = "MeshGenerator == EGenerateStaticMeshLODProcess_MeshGeneratorModes::Solidify || MeshGenerator == EGenerateStaticMeshLODProcess_MeshGeneratorModes::SolidifyAndClose"))
	float WindingThreshold = 0.5f;


	// Morphology settings

	/** Offset distance in the Morpohological Closure operation */
	UPROPERTY(EditAnywhere, Category = Morphology, meta = (UIMin = "0", UIMax = "100", ClampMin = "0", ClampMax = "1000", DisplayName = "Closure Distance",
	EditConditionHides, EditCondition = "MeshGenerator == EGenerateStaticMeshLODProcess_MeshGeneratorModes::SolidifyAndClose"))
	float ClosureDistance = 1.0f;


	// Convex Hull Settings

	/** Whether to subsample input vertices using a regular grid before computing the convex hull */
	UPROPERTY(EditAnywhere, Category = Collision, meta = (EditConditionHides, EditCondition = "MeshGenerator == EGenerateStaticMeshLODProcess_MeshGeneratorModes::ConvexHull"))
	bool bPrefilterVertices = true;

	/** Grid resolution (along the maximum-length axis) for subsampling before computing the convex hull */
	UPROPERTY(EditAnywhere, Category = Collision, meta = (NoSpinbox = "true", EditConditionHides, EditCondition = "MeshGenerator == EGenerateStaticMeshLODProcess_MeshGeneratorModes::ConvexHull && bPrefilterVertices", ClampMin = "4", ClampMax = "100"))
	int PrefilterGridResolution = 10;

};



USTRUCT()
struct FGenerateStaticMeshLODProcess_PreprocessSettings
{
	GENERATED_BODY()

	//
	// NOTE: Customization of widgets for these properties happens in FAutoLODToolDetails::CustomizeDetails(). If you add, remove, or 
	// change any of the properties in this struct, you must also update that function!
	//

	// Filter settings

	/** Group layer to use for filtering out detail before processing */
	UPROPERTY(EditAnywhere, Category = Preprocessing, meta = (DisplayName = "Detail Filter Group Layer"))
	FName FilterGroupLayer = FName();

	// Thicken settings

	// TODO When ThickenWeightMapName is empty the behavior should be that the given ThickenAmount
	// is applied to the entire mesh. This is not yet implemented so we disabled ThickenAmount to
	// reduce confusion.

	/** Weight map used during mesh thickening */
	UPROPERTY(EditAnywhere, Category = Preprocessing, meta = (DisplayName = "Thicken Weight Map"))
	FName ThickenWeightMapName = FName();

	/**
	 * Amount to thicken the mesh prior to Solidifying. The thicken weight map values are multiplied by this value.
	 * Thickening is primarily intended to repair shape erosion that can occur when using the Solidify Mesh Generators.
	 */
	UPROPERTY(EditAnywhere, Category = Preprocessing, meta = (EditCondition = "bIsThickenAmountEnabled", HideEditConditionToggle, UIMin = "0", UIMax = "100", ClampMin = "0", ClampMax = "1000"))
	float ThickenAmount = 0.0f;

	// Transient property, not exposed to the user.
	UPROPERTY(meta = (TransientToolProperty))
	bool bIsThickenAmountEnabled = false;
};




UENUM()
enum class EGenerateStaticMeshLODProcess_SimplifyMethod : uint8
{
	//~NOTE: must keep in sync with UE::GeometryFlow::EMeshSimplifyTargetType
	TriangleCount = 0,
	VertexCount = 1,
	TrianglePercentage = 2,
	GeometricTolerance = 3
};


USTRUCT()
struct FGenerateStaticMeshLODProcess_SimplifySettings
{
	GENERATED_BODY()

	/** Method used to simplify the mesh created in the Mesh Generation step */
	UPROPERTY(EditAnywhere, Category = Simplify, meta = (DisplayName = "Simplification Target"))
	EGenerateStaticMeshLODProcess_SimplifyMethod Method = EGenerateStaticMeshLODProcess_SimplifyMethod::GeometricTolerance;

	/** Target triangle/vertex count after simplification */
	UPROPERTY(EditAnywhere, Category = Simplify, meta = (UIMin = "1", ClampMin = "1", EditConditionHides, EditCondition = "Method == EGenerateStaticMeshLODProcess_SimplifyMethod::TriangleCount || Method == EGenerateStaticMeshLODProcess_SimplifyMethod::VertexCount"))
	int TargetCount = 500;

	/** Target triangle/vertex count after simplification */
	UPROPERTY(EditAnywhere, Category = Simplify, meta = (UIMin = "0", UIMax = "100", ClampMin = "0", ClampMax = "100", EditConditionHides, EditCondition = "Method == EGenerateStaticMeshLODProcess_SimplifyMethod::TrianglePercentage"))
	float TargetPercentage = 5.0;

	/** Deviation in World Units (Centimeters) */
	UPROPERTY(EditAnywhere, Category = Simplify, meta = (UIMin = "0", UIMax = "100", ClampMin = "0", ClampMax = "100", EditConditionHides, EditCondition = "Method == EGenerateStaticMeshLODProcess_SimplifyMethod::GeometricTolerance"))
	float Tolerance = 0.5;
};




UENUM()
enum class EGenerateStaticMeshLODProcess_NormalsMethod : uint8
{
	FromAngleThreshold = 0,
	PerVertex = 1,
	PerTriangle = 2
};


USTRUCT()
struct FGenerateStaticMeshLODProcess_NormalsSettings
{
	GENERATED_BODY()

	/** Type of Normals to generate */
	UPROPERTY(EditAnywhere, Category = Normals, meta = (DisplayName = "Normals Method"))
	EGenerateStaticMeshLODProcess_NormalsMethod Method = EGenerateStaticMeshLODProcess_NormalsMethod::FromAngleThreshold;

	/** Split Normals (ie sharp edge) will be created along mesh edges with opening angles above this threshold */
	UPROPERTY(EditAnywhere, Category = Normals, meta = (UIMin = "0", UIMax = "180", ClampMin = "0", ClampMax = "180", EditConditionHides, EditCondition = "Method == EGenerateStaticMeshLODProcess_NormalsMethod::FromAngleThreshold"))
	float Angle = 60.0;

	static int32 MapMethodType(int32 From, bool bProcessToGraph);
};




UENUM()
enum class EGenerateStaticMeshLODProcess_AutoUVMethod : uint8
{
	PatchBuilder = 0,
	UVAtlas = 1,
	XAtlas = 2
};


USTRUCT()
struct FGenerateStaticMeshLODProcess_UVSettings_PatchBuilder
{
	GENERATED_BODY()

	/** This parameter controls alignment of the initial patches to creases in the mesh */
	UPROPERTY(EditAnywhere, Category = "PatchBuilder", meta = (UIMin = "0.1", UIMax = "2.0", ClampMin = "0.01", ClampMax = "100.0"))
	float CurvatureAlignment = 1.0f;

	/** Number of smoothing steps to apply; this slightly increases distortion but produces more stable results. */
	UPROPERTY(EditAnywhere, Category = "PatchBuilder", AdvancedDisplay, meta = (UIMin = "0", UIMax = "25", ClampMin = "0", ClampMax = "1000"))
	int SmoothingSteps = 5;

	/** Smoothing parameter; larger values result in faster smoothing in each step. */
	UPROPERTY(EditAnywhere, Category = "PatchBuilder", AdvancedDisplay, meta = (UIMin = "0", UIMax = "1.0", ClampMin = "0", ClampMax = "1.0"))
	float SmoothingAlpha = 0.25f;
};

USTRUCT()
struct FGenerateStaticMeshLODProcess_UVSettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = UVGeneration, meta = (DisplayName = "UV Method"))
	EGenerateStaticMeshLODProcess_AutoUVMethod UVMethod = EGenerateStaticMeshLODProcess_AutoUVMethod::PatchBuilder;

	/** Maximum number of charts to create in UVAtlas */
	UPROPERTY(EditAnywhere, Category = "UVAtlas", meta = (DisplayName = "Max Charts", UIMin = 0, UIMax = 1000, EditConditionHides, EditCondition = "UVMethod == EGenerateStaticMeshLODProcess_AutoUVMethod::UVAtlas"))
	int NumUVAtlasCharts = 0;

	/** Number of initial patches mesh will be split into before computing island merging */
	UPROPERTY(EditAnywhere, Category = "PatchBuilder", meta = (UIMin = "1", UIMax = "1000", ClampMin = "1", ClampMax = "99999999", EditConditionHides, EditCondition = "UVMethod == EGenerateStaticMeshLODProcess_AutoUVMethod::PatchBuilder"))
	int NumInitialPatches = 100;

	/** Distortion/Stretching Threshold for island merging - larger values increase the allowable UV stretching */
	UPROPERTY(EditAnywhere, Category = "PatchBuilder", meta = (UIMin = "1.0", UIMax = "5.0", ClampMin = "1.0", EditConditionHides, EditCondition = "UVMethod == EGenerateStaticMeshLODProcess_AutoUVMethod::PatchBuilder"))
	float MergingThreshold = 1.5f;

	/** UV islands will not be merged if their average face normals deviate by larger than this amount */
	UPROPERTY(EditAnywhere, Category = "PatchBuilder", meta = (UIMin = "0.0", UIMax = "90.0", ClampMin = "0.0", ClampMax = "180.0", EditConditionHides, EditCondition = "UVMethod == EGenerateStaticMeshLODProcess_AutoUVMethod::PatchBuilder"))
	float MaxAngleDeviation = 45.0f;

	UPROPERTY(EditAnywhere, Category = "PatchBuilder", meta = (DisplayName = "UV - PatchBuilder", UIMin = "0.0", UIMax = "90.0", ClampMin = "0.0", ClampMax = "180.0", EditConditionHides, EditCondition = "UVMethod == EGenerateStaticMeshLODProcess_AutoUVMethod::PatchBuilder"))
	FGenerateStaticMeshLODProcess_UVSettings_PatchBuilder PatchBuilder;
};




UENUM()
enum class EGenerateStaticMeshLODBakeResolution
{
	Resolution16 = 16 UMETA(DisplayName = "16 x 16"),
	Resolution32 = 32 UMETA(DisplayName = "32 x 32"),
	Resolution64 = 64 UMETA(DisplayName = "64 x 64"),
	Resolution128 = 128 UMETA(DisplayName = "128 x 128"),
	Resolution256 = 256 UMETA(DisplayName = "256 x 256"),
	Resolution512 = 512 UMETA(DisplayName = "512 x 512"),
	Resolution1024 = 1024 UMETA(DisplayName = "1024 x 1024"),
	Resolution2048 = 2048 UMETA(DisplayName = "2048 x 2048"),
	Resolution4096 = 4096 UMETA(DisplayName = "4096 x 4096"),
	Resolution8192 = 8192 UMETA(DisplayName = "8192 x 8192")
};



USTRUCT()
struct FGenerateStaticMeshLODProcess_TextureSettings
{
	GENERATED_BODY()

	/** Resolution for normal map and texture baking */
	UPROPERTY(EditAnywhere, Category = Baking , meta = (DisplayName = "Bake Image Res"))
	EGenerateStaticMeshLODBakeResolution BakeResolution = EGenerateStaticMeshLODBakeResolution::Resolution1024;

	/** How far away from the output mesh to search for input mesh during baking */
	UPROPERTY(EditAnywhere, Category = Baking, meta = (UIMin = "0", UIMax = "100", ClampMin = "0", ClampMax = "1000", DisplayName = "Bake Thickness"))
	float BakeThickness = 5.0f;

	/** Generate an atlassed texture where multiple textures are combined into a single output texture */
	UPROPERTY(EditAnywhere, Category = Baking, meta = (DisplayName = "Combine Textures"))
	bool bCombineTextures = true;
};




UENUM()
enum class EGenerateStaticMeshLODSimpleCollisionGeometryType : uint8
{
	//~NOTE: This must be kept in sync with ESimpleCollisionGeometryType in GenerateSimpleCollisionNode.h
	AlignedBoxes,
	OrientedBoxes,
	MinimalSpheres,
	Capsules,
	ConvexHulls,
	SweptHulls,
	MinVolume,
	None
};


UENUM()
enum class EGenerateStaticMeshLODProjectedHullAxisMode : uint8
{
	//~NOTE: This must be kept in sync with FMeshSimpleShapeApproximation::EProjectedHullAxisMode in MeshSimpleShapeApproximation.h
	X = 0,
	Y = 1,
	Z = 2,
	SmallestBoxDimension = 3,
	SmallestVolume = 4
};



USTRUCT()
struct FGenerateStaticMeshLODProcess_CollisionSettings
{
	GENERATED_BODY()

	// Transient property, not set directly by the user. The user controls a CollisionGroupLayerName dropdown property
	// on the Tool and that value is copied here.
	UPROPERTY(meta = (TransientToolProperty))
	FName CollisionGroupLayerName = TEXT("Default");

	/** Type of simple collision objects to produce */
	UPROPERTY(EditAnywhere, Category = Collision, meta = (DisplayName = "Collision Type"))
	EGenerateStaticMeshLODSimpleCollisionGeometryType CollisionType = EGenerateStaticMeshLODSimpleCollisionGeometryType::ConvexHulls;

	// Convex Hull Settings

	/** Target triangle count for each convex hull after simplification */
	UPROPERTY(EditAnywhere, Category = Collision, meta = (NoSpinbox = "true", DisplayName = "Convex Tri Count",
														  EditConditionHides, EditCondition = "CollisionType == EGenerateStaticMeshLODSimpleCollisionGeometryType::ConvexHulls"))
	int ConvexTriangleCount = 50;

	/** Whether to subsample input vertices using a regular grid before computing the convex hull */
	UPROPERTY(EditAnywhere, Category = Collision, meta = (EditConditionHides, EditCondition = "CollisionType == EGenerateStaticMeshLODSimpleCollisionGeometryType::ConvexHulls"))
	bool bPrefilterVertices = true;

	/** Grid resolution (along the maximum-length axis) for subsampling before computing the convex hull */
	UPROPERTY(EditAnywhere, Category = Collision, meta = (NoSpinbox = "true", EditConditionHides, EditCondition = "CollisionType == EGenerateStaticMeshLODSimpleCollisionGeometryType::ConvexHulls && bPrefilterVertices", UIMin = 4, UIMax = 100))
	int PrefilterGridResolution = 10;


	// Swept Convex Hull Settings

	/** Whether to simplify polygons used for swept convex hulls */
	UPROPERTY(EditAnywhere, Category = Collision, meta = (EditConditionHides, EditCondition = "CollisionType == EGenerateStaticMeshLODSimpleCollisionGeometryType::SweptHulls"))
	bool bSimplifyPolygons = true;

	/** Target minumum edge length for simplified swept convex hulls */
	UPROPERTY(EditAnywhere, Category = Collision, meta = (NoSpinbox = "true", UIMin = "0", UIMax = "10", ClampMin = "0", ClampMax = "100000",
														  EditConditionHides, EditCondition = "CollisionType == EGenerateStaticMeshLODSimpleCollisionGeometryType::SweptHulls"))
	float HullTolerance = 0.1f;

	/** Which axis to sweep along when computing swept convex hulls */
	UPROPERTY(EditAnywhere, Category = Collision, meta = (EditConditionHides, EditCondition = "CollisionType == EGenerateStaticMeshLODSimpleCollisionGeometryType::SweptHulls"))
	EGenerateStaticMeshLODProjectedHullAxisMode SweepAxis = EGenerateStaticMeshLODProjectedHullAxisMode::SmallestVolume;
};



UCLASS(Transient)
class UGenerateStaticMeshLODProcess : public UObject
{
	GENERATED_BODY()
public:

	/**
	 * Initialize the Process, this will:
	 *  - read the Source Mesh, either LOD0 or the HiRes LOD if available
	 *  - Extract all Materials and Textures from the Source Asset and determine what needs to be baked
	 *  - Read all the Source Texture Images that need to be baked
	 *  - call InitializeGenerator() to set up the compute graph
	 */
	[[nodiscard]] bool Initialize(UStaticMesh* SourceMesh, FProgressCancel* Progress = nullptr);


	/** @return the source UStaticMesh passed to Initialize() */
	UStaticMesh* GetSourceStaticMesh() const { return SourceStaticMesh; }
	/** @return the converted source MeshDescription as a FDynamicMesh3, valid after Initialize() finishes */
	const FDynamicMesh3& GetSourceMesh() const { return SourceMesh; }



	//
	// Settings Objects for various stages of the Generator Process
	//

	const FGenerateStaticMeshLODProcess_PreprocessSettings& GetCurrentPreprocessSettings() const { return CurrentSettings_Preprocess; }
	void UpdatePreprocessSettings(const FGenerateStaticMeshLODProcess_PreprocessSettings& NewSettings);

	const FGenerateStaticMeshLODProcessSettings& GetCurrentSettings() const { return CurrentSettings; }
	void UpdateSettings(const FGenerateStaticMeshLODProcessSettings& NewSettings);

	const FGenerateStaticMeshLODProcess_SimplifySettings& GetCurrentSimplifySettings() const { return CurrentSettings_Simplify; }
	void UpdateSimplifySettings(const FGenerateStaticMeshLODProcess_SimplifySettings& NewSettings);

	const FGenerateStaticMeshLODProcess_NormalsSettings& GetCurrentNormalsSettings() const { return CurrentSettings_Normals; }
	void UpdateNormalsSettings(const FGenerateStaticMeshLODProcess_NormalsSettings& NewSettings);

	const FGenerateStaticMeshLODProcess_TextureSettings& GetCurrentTextureSettings() const { return CurrentSettings_Texture; }
	void UpdateTextureSettings(const FGenerateStaticMeshLODProcess_TextureSettings& NewSettings);

	const FGenerateStaticMeshLODProcess_UVSettings& GetCurrentUVSettings() const { return CurrentSettings_UV; }
	void UpdateUVSettings(const FGenerateStaticMeshLODProcess_UVSettings& NewSettings);

	const FGenerateStaticMeshLODProcess_CollisionSettings& GetCurrentCollisionSettings() const { return CurrentSettings_Collision; }
	void UpdateCollisionSettings(const FGenerateStaticMeshLODProcess_CollisionSettings& NewSettings);


	//
	// Material Queries and Constraints
	//

	/** @return the set of Materials that were identified on the Source Asset */
	TArray<UMaterialInterface*> GetSourceBakeMaterials() const;

	enum class EMaterialBakingConstraint
	{
		NoConstraint = 0,
		UseExistingMaterial = 1
	};

	/** Configure the constraint on the specified Source Material */
	void UpdateSourceBakeMaterialConstraint(UMaterialInterface* Material, EMaterialBakingConstraint Constraint);


	//
	// Texture Queries and Constraints
	//

	/** @return the set of Textures that were identified as being parameters to the Source Materials, that are identified as potentially needing to be baked to the Derived Asset UVs */
	TArray<UTexture2D*> GetSourceBakeTextures() const;


	enum class ETextureBakingConstraint
	{
		NoConstraint = 0,
		UseExistingTexture = 1
	};

	/** Configure the constraint on the specified Source Texture */
	void UpdateSourceBakeTextureConstraint(UTexture2D* Texture, ETextureBakingConstraint Constraint);


	//
	// Output Path / Naming / etc Configuration
	//

	/**
	 * Configure the output Name and Suffix for the Mesh Asset. The various strings below will be
	 * computed based on this input
	 */
	void UpdateDerivedPathName(const FString& NewAssetBaseName, const FString& NewAssetSuffix);


	const FString& GetSourceAssetPath() const { return SourceAssetPath; }
	const FString& GetSourceAssetFolder() const { return SourceAssetFolder; }
	const FString& GetSourceAssetName() const { return SourceAssetName; }
	static FString GetDefaultDerivedAssetSuffix() { return TEXT("_AutoLOD"); }
	const FString& GetDerivedAssetName() const { return DerivedAssetName; }


	/**
	 * Given the above configuration, run the Generator to compute the derived Meshes and Textures
	 */

	bool ComputeDerivedSourceData(FProgressCancel* Progress);

	/** The Generated Mesh */
	const FDynamicMesh3& GetDerivedLOD0Mesh() const { return DerivedLODMesh; }
	/** Tangents for the Generated Mesh */
	const UE::Geometry::FMeshTangentsd& GetDerivedLOD0MeshTangents() const { return DerivedLODMeshTangents; }
	/** Collision geometry for the Generated Mesh */
	const UE::Geometry::FSimpleShapeSet3d& GetDerivedCollision() const { return DerivedCollision; }

	struct FPreviewMaterials
	{
		TArray<UMaterialInterface*> Materials;
		TArray<UTexture2D*> Textures;
	};
	/** Set of Materials and Textures to be applied to the Generated Mesh */
	void GetDerivedMaterialsPreview(FPreviewMaterials& MaterialSetOut);


	//
	// Final Output Options
	//

	/**
	 * Creates new UStaticMesh Asset, Material Instances, and Textures
	 */
	virtual bool WriteDerivedAssetData();


	/**
	 * Update the input UStaticMesh Asset (ie the asset passed to Initialize)
	 * @param bSetNewHDSourceAsset If true, the original source mesh copied to the UStaticMesh's HiRes SourceModel slot
	 */
	virtual void UpdateSourceAsset(bool bSetNewHDSourceAsset = false);



public:
	/**
	 * This FCriticalSection is used to co-ordinate evaluation of the internal compute graph
	 * by external classes. Generally while doing things like changing settings, reading output, etc,
	 * the internal compute graph cannot be executing. Often this needs to be locked across multiple
	 * function calls, so it is exposed publicly.  (TODO: clean this up...)
	 */
	FCriticalSection GraphEvalCriticalSection;

protected:

	UPROPERTY()
	TObjectPtr<UStaticMesh> SourceStaticMesh;

	FString SourceAssetPath;
	FString SourceAssetFolder;
	FString SourceAssetName;

	// if true, we are building new LOD0 from the StaticMesh HiRes SourceModel, instead of from the mesh in LOD0
	bool bUsingHiResSource = false;

	// copy of input MeshDescription with autogenerated attributes computed
	TSharedPtr<FMeshDescription> SourceMeshDescription;
	// SourceMeshDescription converted to FDynamicMesh3
	FDynamicMesh3 SourceMesh;

	struct FTextureInfo
	{
		UTexture2D* Texture = nullptr;
		FName ParameterName;
		UE::Geometry::FImageDimensions Dimensions;
		UE::Geometry::TImageBuilder<FVector4f> Image;
		bool bIsNormalMap = false;
		bool bIsDefaultTexture = false;
		bool bShouldBakeTexture = false;
		bool bIsUsedInMultiTextureBaking = false; 
		ETextureBakingConstraint Constraint = ETextureBakingConstraint::NoConstraint;
	};

	int SelectTextureToBake(const TArray<FTextureInfo>& TextureInfos) const;

	// Information about one of the input StaticMesh Materials. Computed in Initialize() and not modified afterwards
	struct FSourceMaterialInfo
	{
		FStaticMaterial SourceMaterial;
		TArray<FTextureInfo> SourceTextures;

		bool bHasNormalMap = false;						// if true, Material has an exposed NormalMap input texture parameter
		bool bHasTexturesToBake = false;				// if true, Material has at least one SourceTexture that should be baked
		bool bIsReusable = false;						// if true, Material doesn't need any texture baking and can be re-used by LOD0
		bool bIsPreviouslyGeneratedMaterial = false;	// if true, this Material was previously generated by AutoLOD and should be discarded. 
														// Currently inferred from material being in LOD0 but not HiRes Source

		EMaterialBakingConstraint Constraint = EMaterialBakingConstraint::NoConstraint;
	};

	// list of initial Source Materials, length equivalent to StaticMesh.StaticMaterials
	TArray<FSourceMaterialInfo> SourceMaterials;

	FString DerivedSuffix;
	FString DerivedAssetPath;
	FString DerivedAssetFolder;
	FString DerivedAssetName;
	FString DerivedAssetNameNoSuffix;
	

	FDynamicMesh3 DerivedLODMesh;				// the new generated LOD0 mesh
	UE::Geometry::FMeshTangentsd DerivedLODMeshTangents;		// Tangents for DerivedLODMesh
	UE::Geometry::FSimpleShapeSet3d DerivedCollision;			// Simple Collision for DerivedLODMesh

	// Texture set potentially required by output Material set
	UE::GeometryFlow::FNormalMapImage DerivedNormalMapImage;	// Normal Map
	TArray<TUniquePtr<UE::GeometryFlow::FTextureImage>> DerivedTextureImages;	// generated Textures
	UE::GeometryFlow::FTextureImage DerivedMultiTextureBakeImage;
	TMap<UTexture2D*, int32> SourceTextureToDerivedTexIndex;	// mapping from input Textures to DerivedTextureImages index

	// Information about an output Material
	struct FDerivedMaterialInfo
	{
		int32 SourceMaterialIndex = -1;				// Index into SourceMaterials
		bool bUseSourceMaterialDirectly = false;	// If true, do not create/use a Derived Material, directly re-use SourceMaterials[SourceMaterialIndex]

		FStaticMaterial DerivedMaterial;			// points to generated Material
		TArray<FTextureInfo> DerivedTextures;		// List of generated Textures
	};

	// list of generated/Derived Materials. Length is the same as SourceMaterials and indices correspond, however some Derived Materials may not be initialized (if Source is reusable or generated)
	TArray<FDerivedMaterialInfo> DerivedMaterials;

	// This list is for accumulating derived UTexture2D's created during WriteDerivedTextures(). We have to
	// maintain uproperty references to these or they may be garbage collected
	UPROPERTY()
	TSet<TObjectPtr<UTexture2D>> AllDerivedTextures;

	// Derived Normal Map
	UPROPERTY()
	TObjectPtr<UTexture2D> DerivedNormalMapTex;

	// For each material participating in multi-texture baking, the parameter name of the texture
	TMap<int32, FName> MultiTextureParameterName;

	UPROPERTY()
	TObjectPtr<UTexture2D> DerivedMultiTextureBakeResult;


	TUniquePtr<FGenerateMeshLODGraph> Generator;			// active LODGenerator Graph
	bool InitializeGenerator();

	FGenerateStaticMeshLODProcess_PreprocessSettings CurrentSettings_Preprocess;
	FGenerateStaticMeshLODProcessSettings CurrentSettings;
	FGenerateStaticMeshLODProcess_SimplifySettings CurrentSettings_Simplify;
	FGenerateStaticMeshLODProcess_NormalsSettings CurrentSettings_Normals;
	FGenerateStaticMeshLODProcess_TextureSettings CurrentSettings_Texture;
	FGenerateStaticMeshLODProcess_UVSettings CurrentSettings_UV;
	FGenerateStaticMeshLODProcess_CollisionSettings CurrentSettings_Collision;
	
	// This value will be stored in Asset Metadata for each generated Asset Texture, Material/MIC, and StaticMesh,
	// under the key StaticMeshLOD.GenerationGUID.
	FString DerivedAssetGUIDKey;

	bool WriteDerivedTexture(UTexture2D* SourceTexture, UTexture2D* DerivedTexture, bool bCreatingNewStaticMeshAsset);
	bool WriteDerivedTexture(UTexture2D* DerivedTexture, FString BaseTexName, bool bCreatingNewStaticMeshAsset);
	void WriteDerivedTextures(bool bCreatingNewStaticMeshAsset);
	void WriteDerivedMaterials(bool bCreatingNewStaticMeshAsset);
	void UpdateMaterialTextureParameters(UMaterialInstanceConstant* Material, FDerivedMaterialInfo& DerivedMaterialInfo);
	void WriteDerivedStaticMeshAsset();

	void UpdateSourceStaticMeshAsset(bool bSetNewHDSourceAsset);

	void UpdateMaterialTextureParameters(UMaterialInstanceDynamic* Material, const FSourceMaterialInfo& SourceMaterialInfo,
		const TMap<UTexture2D*,UTexture2D*>& PreviewTextures, UTexture2D* PreviewNormalMap);

	// Return true if the given path corresponds to a material or texture in SourceMaterials
	bool IsSourceAsset(const FString& AssetPath) const;
};
