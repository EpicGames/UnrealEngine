// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

DECLARE_LOG_CATEGORY_EXTERN(LogMeshLODToolset, Log, All);

class IAssetTypeActions;

class FMeshLODToolsetModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	void OnPostEngineInit();

	// registered asset actions
	TArray<TSharedRef<IAssetTypeActions>> RegisteredAssetTypeActions;

private:
	/** StaticClass is not safe on shutdown, so we cache the name, and use this to unregister on shut down */
	TArray<FName> ClassesToUnregisterOnShutdown;

};
