// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

/// 
/// Detail customization for UGenerateStaticMeshLODAssetTool (AKA AutoLOD) properties
///
 
class FAutoLODToolDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};

