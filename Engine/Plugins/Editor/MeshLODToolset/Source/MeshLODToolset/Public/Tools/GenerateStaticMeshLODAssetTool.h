// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseTools/MultiSelectionMeshEditingTool.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Graphs/GenerateStaticMeshLODProcess.h"
#include "GenerateStaticMeshLODAssetTool.generated.h"

#define UE_API MESHLODTOOLSET_API

class UCollisionGeometryVisualizationProperties;
class UPhysicsObjectToolPropertySet;
class UPreviewGeometry;
namespace UE::Geometry { class IDynamicMeshOperatorFactory; }


// predeclarations
struct FMeshDescription;
class UDynamicMeshComponent;
class UMeshOpPreviewWithBackgroundCompute;
class UGenerateStaticMeshLODAssetTool;
class UStaticMeshLODGenerationSettings;
namespace GenerateStaticMeshLODAssetLocals
{
	class FGenerateStaticMeshLODAssetOperatorFactory;
}

UENUM()
enum class EGenerateLODAssetOutputMode : uint8
{
	CreateNewAsset = 0,
	UpdateExistingAsset = 1
};


/**
 * Tool builder
 */
UCLASS(MinimalAPI)
class UGenerateStaticMeshLODAssetToolBuilder : public UMultiSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	bool bUseAssetEditorMode = false;
	bool bInRestrictiveMode = false;

	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};



UCLASS(MinimalAPI)
class UGenerateStaticMeshLODAssetToolOutputProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Whether to modify the static mesh in place or create a new one. */
	UPROPERTY(EditAnywhere, Category = "Output Options", meta = (TransientToolProperty, HideEditConditionToggle, EditCondition = "bShowOutputMode"))
	EGenerateLODAssetOutputMode OutputMode = EGenerateLODAssetOutputMode::CreateNewAsset;

	/** Base name for newly-generated asset */
	UPROPERTY(EditAnywhere, Category = "Output Options", meta = (TransientToolProperty, EditConditionHides, EditCondition = "OutputMode == EGenerateLODAssetOutputMode::CreateNewAsset"))
	FString NewAssetName;

	/** If the Asset doesn't already have a HiRes source, store the input mesh as the HiRes source */
	UPROPERTY(EditAnywhere, Category = "Output Options", meta = (EditConditionHides, EditCondition = "OutputMode == EGenerateLODAssetOutputMode::UpdateExistingAsset"))
	bool bSaveInputAsHiResSource = true;

	/** Suffix to append to newly-generated Asset (Meshes, Textures, Materials, etc) */
	UPROPERTY(EditAnywhere, Category = "Output Options", meta = (TransientToolProperty))
	FString GeneratedSuffix = TEXT("_AutoLOD");

	/** If false, then OutputMode will not be shown in DetailsView panels (otherwise no effect) */
	UPROPERTY(meta = (TransientToolProperty))
	bool bShowOutputMode = true;
};




UENUM()
enum class EGenerateLODAssetToolPresetAction : uint8
{
	ReadFromPreset,
	WriteToPreset
};


UCLASS(MinimalAPI)
class UGenerateStaticMeshLODAssetToolPresetProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<UGenerateStaticMeshLODAssetTool> ParentTool;
	void Initialize(UGenerateStaticMeshLODAssetTool* ParentToolIn) { ParentTool = ParentToolIn; }
	UE_API virtual void PostAction(EGenerateLODAssetToolPresetAction Action);

public:
	/** Preset Asset represents a set of Saved Settings for this Tool */
	UPROPERTY(EditAnywhere, Category = Preset, meta = (DisplayName = "Settings Preset"))
	TWeakObjectPtr<UStaticMeshLODGenerationSettings> Preset;

	/** Read the current Tool settings from the Preset Asset */
	UFUNCTION(CallInEditor, Category = Preset, meta = (DisplayPriority = 0))
	void ReadFromPreset()
	{
		PostAction(EGenerateLODAssetToolPresetAction::ReadFromPreset);
	}

	/** Save the current Tool settings to the Preset Asset */
	UFUNCTION(CallInEditor, Category = Preset, meta = (DisplayPriority = 1))
	void WriteToPreset()
	{
		PostAction(EGenerateLODAssetToolPresetAction::WriteToPreset);
	}
};



/**
 * Standard properties
 */
UCLASS(MinimalAPI)
class UGenerateStaticMeshLODAssetToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Generator Configuration")
	FGenerateStaticMeshLODProcess_PreprocessSettings Preprocessing;

	UPROPERTY(EditAnywhere, Category = "Generator Configuration", meta = (ExpandByDefault))
	FGenerateStaticMeshLODProcessSettings MeshGeneration;

	UPROPERTY(EditAnywhere, Category = "Generator Configuration", meta = (ExpandByDefault))
	FGenerateStaticMeshLODProcess_SimplifySettings Simplification;

	UPROPERTY(EditAnywhere, Category = "Generator Configuration", meta = (ExpandByDefault))
	FGenerateStaticMeshLODProcess_NormalsSettings Normals;

	UPROPERTY(EditAnywhere, Category = "Generator Configuration", meta = (ExpandByDefault))
	FGenerateStaticMeshLODProcess_TextureSettings TextureBaking;

	UPROPERTY(EditAnywhere, Category = "Generator Configuration")
	FGenerateStaticMeshLODProcess_UVSettings UVGeneration;

	UPROPERTY(EditAnywhere, Category = "Generator Configuration")
	FGenerateStaticMeshLODProcess_CollisionSettings SimpleCollision;


	// ------------
	// Code copied from UPolygroupLayersProperties

	/** Group layer to use for partitioning the mesh for simple collision generation */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (GetOptions = GetGroupLayersFunc))
	FName CollisionGroupLayerName = TEXT("Default");

	// this function is called provide set of available group layers
	UFUNCTION()
	TArray<FString> GetGroupLayersFunc()
	{
		return GroupLayersList;
	}

	// internal list used to implement above
	UPROPERTY(meta = (TransientToolProperty))
	TArray<FString> GroupLayersList;

	void InitializeGroupLayers(const FDynamicMesh3* Mesh)
	{
		GroupLayersList.Reset();
		GroupLayersList.Add(TEXT("Default"));		// always have standard group
		if (Mesh->Attributes())
		{
			for (int32 k = 0; k < Mesh->Attributes()->NumPolygroupLayers(); k++)
			{
				FName Name = Mesh->Attributes()->GetPolygroupLayer(k)->GetName();
				GroupLayersList.Add(Name.ToString());
			}
		}

		if (GroupLayersList.Contains(CollisionGroupLayerName.ToString()) == false)		// discard restored value if it doesn't apply
		{
			CollisionGroupLayerName = FName(GroupLayersList[0]);
		}
	}

};


UENUM()
enum class EGenerateStaticMeshLOD_BakeConstraint
{
	NoConstraint = 0,
	DoNotBake = 1
};

USTRUCT()
struct FGenerateStaticMeshLOD_TextureConfig
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = Texture)
	TObjectPtr<UTexture2D> Texture = nullptr;

	UPROPERTY(EditAnywhere, Category = Texture)
	EGenerateStaticMeshLOD_BakeConstraint Constraint = EGenerateStaticMeshLOD_BakeConstraint::NoConstraint;

	bool operator==(const FGenerateStaticMeshLOD_TextureConfig& Other) const
	{
		return Texture == Other.Texture && Constraint == Other.Constraint;
	}
};


USTRUCT()
struct FGenerateStaticMeshLOD_MaterialConfig
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = Material, meta = (TransientToolProperty, NoResetToDefault))
	TObjectPtr<UMaterialInterface> Material = nullptr;

	UPROPERTY(EditAnywhere, Category = Texture, meta = (TransientToolProperty, NoResetToDefault))
	EGenerateStaticMeshLOD_BakeConstraint Constraint = EGenerateStaticMeshLOD_BakeConstraint::NoConstraint;

	bool operator==(const FGenerateStaticMeshLOD_MaterialConfig& Other) const
	{
		return Material == Other.Material && Constraint == Other.Constraint;
	}
};



UCLASS(MinimalAPI)
class UGenerateStaticMeshLODAssetToolTextureProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = "Source Materials Configuration", meta = (TransientToolProperty, NoResetToDefault))
	TArray<FGenerateStaticMeshLOD_MaterialConfig> Materials;

	UPROPERTY(EditAnywhere, Category = "Source Textures Configuration", meta = (TransientToolProperty, NoResetToDefault))
	TArray<FGenerateStaticMeshLOD_TextureConfig> Textures;

	UPROPERTY(VisibleAnywhere, Category = "Baked Texture Previews", meta = (TransientToolProperty, NoResetToDefault))
	TArray<TObjectPtr<UTexture2D>> PreviewTextures;
};




/**
 * Simple tool to combine multiple meshes into a single mesh asset
 */
UCLASS(MinimalAPI)
class UGenerateStaticMeshLODAssetTool : public UMultiSelectionMeshEditingTool
{
	GENERATED_BODY()

public:

	// Enable UI Customization for running this Tool in the Static Mesh Asset Editor. Must be called before Setup.
	UE_API virtual void SetUseAssetEditorMode(bool bEnable);

	// Enable restrictive mode. Must be called before Setup.
	UE_API virtual void SetRestrictiveMode(bool bEnable);

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime);

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	UE_API virtual void RequestPresetAction(EGenerateLODAssetToolPresetAction ActionType);

protected:

	UPROPERTY()
	TObjectPtr<UGenerateStaticMeshLODAssetToolOutputProperties> OutputProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UGenerateStaticMeshLODAssetToolProperties> BasicProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UGenerateStaticMeshLODAssetToolPresetProperties> PresetProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UGenerateStaticMeshLODAssetToolTextureProperties> TextureProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UCollisionGeometryVisualizationProperties> CollisionVizSettings = nullptr;



protected:

	friend class GenerateStaticMeshLODAssetLocals::FGenerateStaticMeshLODAssetOperatorFactory;

	UPROPERTY()
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> PreviewWithBackgroundCompute = nullptr;

	UPROPERTY()
	TArray<TObjectPtr<UTexture2D>> PreviewTextures;

	UPROPERTY()
	TArray<TObjectPtr<UMaterialInterface>> PreviewMaterials;


protected:

	UPROPERTY()
	TObjectPtr<UPhysicsObjectToolPropertySet> ObjectData = nullptr;

	UPROPERTY()
	TObjectPtr<UPreviewGeometry> CollisionPreview;

protected:
	bool bIsInAssetEditorMode = false;
	bool bRestrictiveMode = false;

	UPROPERTY()
	TObjectPtr<UGenerateStaticMeshLODProcess> GenerateProcess;

	TUniquePtr<UE::Geometry::IDynamicMeshOperatorFactory> OpFactory;

	UE_API void OnSettingsModified();

	UE_API void CreateNewAsset();
	UE_API void UpdateExistingAsset();

	UE_API void OnPresetSelectionChanged();

	UE_API bool ValidateSettings() const;
};

#undef UE_API
