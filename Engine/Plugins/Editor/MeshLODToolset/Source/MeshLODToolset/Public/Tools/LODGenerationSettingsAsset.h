// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graphs/GenerateStaticMeshLODProcess.h"

#include "LODGenerationSettingsAsset.generated.h"


/**
 * UStaticMeshLODGenerationSettings is intended to be a stored version of the settings used
 * by UGenerateStaticMeshLODProcess (and the associated UGenerateStaticMeshLODAssetTool). 
 * This UObject is exposed as an Asset type in the Editor via UStaticMeshLODGenerationSettingsFactory.
 * 
 * The Tool uses these serialized settings as a 'Preset', ie the user can save a set
 * of configured settings, or load previously-saved settings. 
 */
UCLASS(MinimalAPI, Blueprintable)
class UStaticMeshLODGenerationSettings : public UObject
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
public:
	UPROPERTY(EditAnywhere, Category=Settings)
	FGenerateStaticMeshLODProcess_PreprocessSettings Preprocessing;

	UPROPERTY(EditAnywhere, Category=Settings)
	FGenerateStaticMeshLODProcessSettings MeshGeneration;

	UPROPERTY(EditAnywhere, Category=Settings)
	FGenerateStaticMeshLODProcess_SimplifySettings Simplification;

	UPROPERTY(EditAnywhere, Category=Settings)
	FGenerateStaticMeshLODProcess_NormalsSettings Normals;

	UPROPERTY(EditAnywhere, Category=Settings)
	FGenerateStaticMeshLODProcess_TextureSettings TextureBaking;

	UPROPERTY(EditAnywhere, Category=Settings)
	FGenerateStaticMeshLODProcess_UVSettings UVGeneration;

	UPROPERTY(EditAnywhere, Category=Settings)
	FGenerateStaticMeshLODProcess_CollisionSettings SimpleCollision;
#endif
};
