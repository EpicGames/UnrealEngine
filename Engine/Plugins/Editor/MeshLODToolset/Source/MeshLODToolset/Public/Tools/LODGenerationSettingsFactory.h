// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "LODGenerationSettingsFactory.generated.h"

/**
 * Asset Factory for UStaticMeshLODGenerationSettings, which is used to save settings for
 * the AutoLOD Tool/Process as an Asset in the Editor
 */
UCLASS(MinimalAPI, hidecategories = Object)
class UStaticMeshLODGenerationSettingsFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual uint32 GetMenuCategories() const override;
	virtual FText GetDisplayName() const override;
	virtual FString GetDefaultNewAssetName() const override;
	virtual bool ShouldShowInNewMenu() const override { return true; }
	//~ Begin UFactory Interface	
};
