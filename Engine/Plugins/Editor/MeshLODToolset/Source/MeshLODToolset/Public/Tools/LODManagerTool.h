// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseTools/MultiSelectionMeshEditingTool.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Engine/StaticMesh.h"
#include "LODManagerTool.generated.h"

#define UE_API MESHLODTOOLSET_API

class UPreviewGeometry;
class UPreviewMesh;

// predeclarations
class UMaterialInterface;
class ULODManagerTool;

/**
 *
 */
UCLASS(MinimalAPI)
class ULODManagerToolBuilder : public UMultiSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};


USTRUCT()
struct FLODManagerLODInfo
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = LODInformation, meta=(NoResetToDefault))
	int32 VertexCount = 0;

	UPROPERTY(VisibleAnywhere, Category = LODInformation, meta=(NoResetToDefault))
	int32 TriangleCount = 0;
};



UCLASS(MinimalAPI)
class ULODManagerLODProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	UPROPERTY(VisibleAnywhere, Category = LODInformation, meta = (NoResetToDefault))
	TArray<FLODManagerLODInfo> SourceLODs;

	UPROPERTY(VisibleAnywhere, Category = LODInformation, meta = (NoResetToDefault))
	TArray<FLODManagerLODInfo> HiResSource;


	UPROPERTY(VisibleAnywhere, Category = LODInformation, meta = (NoResetToDefault))
	TArray<FLODManagerLODInfo> RenderLODs;

	UPROPERTY(VisibleAnywhere, Category = Nanite, meta = (NoResetToDefault, DisplayName="Enabled"))
	bool bNaniteEnabled = false;

	// Percentage of triangles kept by Nanite
	UPROPERTY(VisibleAnywhere, Category = Nanite, meta = (NoResetToDefault))
	float KeepTrianglePercent = 0.0f;

	UPROPERTY(VisibleAnywhere, Category = Materials, meta = (NoResetToDefault))
	TArray<FStaticMaterial> Materials;

};



UCLASS(MinimalAPI)
class ULODManagerPreviewLODProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** LOD to visualise. Default option is equivalent to disabling the Tool, RenderData is the mesh used for rendering derived from the SourceModel (possibly simplified) */
	UPROPERTY(EditAnywhere, Category = LODPreview, meta = (DisplayName = "Show LOD", NoResetToDefault, GetOptions = GetLODNamesFunc))
	FString VisibleLOD;

	UFUNCTION()
	const TArray<FString>& GetLODNamesFunc() const { return LODNamesList; }

	UPROPERTY(meta = (TransientToolProperty))
	TArray<FString> LODNamesList;

	/** Control whether mesh borders are displayed */
	UPROPERTY(EditAnywhere, DisplayName = "Show Borders", Category = LODPreview)
	bool bShowSeams = true;

};







UENUM()
enum class ELODManagerToolActions
{
	NoAction,

	MoveHiResToLOD0,
	DeleteHiResSourceModel,
	RemoveUnreferencedMaterials
};



UCLASS(MinimalAPI)
class ULODManagerActionPropertySet : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<ULODManagerTool> ParentTool;

	void Initialize(ULODManagerTool* ParentToolIn) { ParentTool = ParentToolIn; }
	UE_API void PostAction(ELODManagerToolActions Action);

};


UCLASS(MinimalAPI)
class ULODManagerHiResSourceModelActions : public ULODManagerActionPropertySet
{
	GENERATED_BODY()
public:
	/** Move the HiRes Source Model to LOD0 */
	UFUNCTION(CallInEditor, Category = HiResSourceModel, meta = (DisplayPriority = 0))
	void MoveToLOD0()
	{
		PostAction(ELODManagerToolActions::MoveHiResToLOD0);
	}

	/** Delete the HiRes Source Model */
	UFUNCTION(CallInEditor, Category = HiResSourceModel, meta = (DisplayPriority = 1))
	void Delete()
	{
		PostAction(ELODManagerToolActions::DeleteHiResSourceModel);
	}
};


UCLASS(MinimalAPI)
class ULODManagerMaterialActions : public ULODManagerActionPropertySet
{
	GENERATED_BODY()
public:
	/** Discard any Materials that are not referenced by any LOD */
	UFUNCTION(CallInEditor, Category = MaterialSet, meta = (DisplayPriority = 0))
	void CleanMaterials()
	{
		PostAction(ELODManagerToolActions::RemoveUnreferencedMaterials);
	}
};

// forward
namespace UE { namespace Geometry { namespace LODManagerHelper {
class FLODManagerToolChange; 
struct FProxyLODState;
class FDynamicMeshLODCache;

} } }


UINTERFACE(MinimalAPI)
class ULODManagerToolChangeTarget : public UInterface
{
	GENERATED_BODY()
};
/**
 * IMeshVertexCommandChangeTarget is an interface which is used to apply a FLODManagerToolChange
 */
class ILODManagerToolChangeTarget
{
	GENERATED_BODY()
public:
	virtual void ApplyChange(const UE::Geometry::LODManagerHelper::FLODManagerToolChange* Change, bool bRevert) = 0;
};

/**
 * Mesh Attribute Editor Tool
 */
UCLASS(MinimalAPI)
class ULODManagerTool : public UMultiSelectionMeshEditingTool, public ILODManagerToolChangeTarget
{
	GENERATED_BODY()

public:
	UE_API ULODManagerTool();

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }

	UE_API virtual void RequestAction(ELODManagerToolActions ActionType);

	UE_API virtual void ApplyChange(const UE::Geometry::LODManagerHelper::FLODManagerToolChange* Change, bool bRevert) override;

protected:

	UPROPERTY()
	TObjectPtr<ULODManagerLODProperties> LODInfoProperties;

	UPROPERTY()
	TObjectPtr<ULODManagerPreviewLODProperties> LODPreviewProperties;

	UPROPERTY()
	TObjectPtr<ULODManagerHiResSourceModelActions> HiResSourceModelActions;

	UPROPERTY()
	TObjectPtr<ULODManagerMaterialActions> MaterialActions;



public:
	UFUNCTION()
	UE_API void DeleteHiResSourceModel();

	UFUNCTION()
	UE_API void MoveHiResToLOD0();

	UFUNCTION()
	UE_API void RemoveUnreferencedMaterials();


	struct FLODName
	{
		int32 SourceModelIndex = -1;
		int32 RenderDataIndex = -1;
		int32 OtherIndex = -1;
		bool IsValid() const { return ! (SourceModelIndex == -1 && RenderDataIndex == -1 && OtherIndex == -1); }
	};

	struct FLODMeshInfo
	{
		TSharedPtr<UE::Geometry::FDynamicMesh3> Mesh;
		TArray<int> BoundaryEdges;
	};


protected:
	UE_API UStaticMesh* GetSingleStaticMesh();

	ELODManagerToolActions PendingAction = ELODManagerToolActions::NoAction;
	

	bool bLODInfoValid = false;
	// captures the material list and triangle and vertex counts for the current configuration of lods
	UE_API void UpdateLODInfo();

	// maps pretty name in UI to description of the LOD
	TMap<FString, FLODName> ActiveLODNames;
	UE_API void UpdateLODNames();

	UPROPERTY()
	TObjectPtr<UPreviewMesh> LODPreview;

	UPROPERTY()
	TObjectPtr<UPreviewGeometry> LODPreviewLines;

	bool bPreviewLODValid = false;
	UE_API void UpdatePreviewLOD();
	UE_API void UpdatePreviewLines(FLODMeshInfo& LODMeshInfo);
	UE_API void ClearPreviewLines();

	// returns the requested LOD as a dynamic mesh along with boundary edges.  Has internal cache
	UE_API TUniquePtr<ULODManagerTool::FLODMeshInfo> GetLODMeshInfo(const FLODName& LODName);

	// For undo / redo system with our custom changes.  
	TUniquePtr< UE::Geometry::LODManagerHelper::FLODManagerToolChange > ActiveChange;
	UE_API void BeginChange(FText TransactionName);
	UE_API void EndChange();

	// state information used for undo within the tool
	TUniquePtr< UE::Geometry::LODManagerHelper::FProxyLODState>  ProxyLODState;

	// cache of dynamic mesh representation for each lod, imported and renderdata.
	TUniquePtr< UE::Geometry::LODManagerHelper::FDynamicMeshLODCache > DynamicMeshCache;
};

#undef UE_API
